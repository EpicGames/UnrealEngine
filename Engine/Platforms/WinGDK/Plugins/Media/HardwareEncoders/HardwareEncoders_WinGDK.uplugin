{
	"FileVersion": 3,
	"bIsPluginExtension": true,
	"Modules": [
		{
			"Name": "EncoderAMF",
			"Type": "Runtime",
			"PlatformAllowList": [
				"WinGDK"
			]
		},
		{
			"Name": "EncoderNVENC",
			"Type": "Runtime",
			"PlatformAllowList": [
				"WinGDK"
			]
		}
	]
}