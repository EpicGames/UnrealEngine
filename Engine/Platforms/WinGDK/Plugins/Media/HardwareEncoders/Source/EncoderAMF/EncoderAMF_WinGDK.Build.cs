// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class EncoderAMF_WinGDK : EncoderAMF
	{
		public EncoderAMF_WinGDK(ReadOnlyTargetRules Target) : base(Target)
		{
		}
	}
}
