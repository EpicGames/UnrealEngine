{
	"FileVersion": 3,
	"Version": 0.2,
	"VersionName": "0.2",
	"FriendlyName": "HardwareEncoders",
	"Description": "Adds support of hardware encoders to AVEncoder",
	"Category": "Encoders",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "EncoderAMF",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit",
			"PlatformAllowList": [ "Win64", "Linux" ]
		},
		{
			"Name": "EncoderNVENC",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit",
			"PlatformAllowList": [ "Win64", "Linux" ]
		},
	]
}
