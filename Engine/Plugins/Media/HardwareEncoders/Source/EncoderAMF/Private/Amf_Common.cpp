// Copyright Epic Games, Inc. All Rights Reserved.

#include "Amf_Common.h"
#include "CoreGlobals.h"
#include "HAL/PlatformProcess.h"
#include "Misc/ScopeLock.h"
#include "RHIDefinitions.h"
#include "Trace/Trace.inl"
#include <components/VideoEncoderVCE.h>
#include <core/Factory.h>
#include <core/Result.h>
#include <core/Version.h>
#include <vulkan_core.h>

DEFINE_LOG_CATEGORY(LogEncoderAMF);

namespace AVEncoder
{
	FCriticalSection FAmfCommon::ProtectSingleton;
	FAmfCommon FAmfCommon::Singleton;

	// attempt to load Amf
	FAmfCommon &FAmfCommon::Setup()
	{
		FScopeLock Guard(&ProtectSingleton);
		if (!Singleton.bWasSetUp)
		{
			Singleton.bWasSetUp = true;
			Singleton.SetupAmfFunctions();
		}
		return Singleton;
	}

	// shutdown - release loaded dll
	void FAmfCommon::Shutdown()
	{
		FScopeLock Guard(&ProtectSingleton);
		if (Singleton.bWasSetUp)
		{
			Singleton.bWasSetUp = false;
			Singleton.bIsAvailable = false;

			if (Singleton.AmfContext)
			{
				Singleton.AmfContext->Terminate();
				Singleton.AmfContext = nullptr;
			}

			Singleton.AmfFactory = nullptr;

			if (Singleton.DllHandle)
			{
				FPlatformProcess::FreeDllHandle(Singleton.DllHandle);
				Singleton.DllHandle = nullptr;
			}
		}
	}

	bool FAmfCommon::CreateEncoder(amf::AMFComponentPtr &outEncoder)
	{
		AMF_RESULT res = AmfFactory->CreateComponent(AmfContext, AMFVideoEncoderVCE_AVC, &outEncoder);
		if (res != AMF_OK)
		{
			UE_LOGF(LogEncoderAMF, Error, "AMF failed to create Encoder component with code: %d", res);
			return false;
		}
		return true;
	}

	void FAmfCommon::SetupAmfFunctions()
	{
		check(!bIsAvailable);

#if PLATFORM_WINDOWS
		// Early out on non-supported windows versions
		if (!FPlatformMisc::VerifyWindowsVersion(6, 2))
		{
			return;
		}
#endif

#ifdef AMF_DLL_NAMEA
// To avoid a warning during tests we manually call dlopen on Linux as this is how we currently determine if AMF is avaliable
#if PLATFORM_LINUX && PLATFORM_DESKTOP
		DllHandle = dlopen( AMF_DLL_NAMEA, RTLD_LAZY | RTLD_NOLOAD | RTLD_GLOBAL );
#else 
		DllHandle = FPlatformProcess::GetDllHandle(TEXT(AMF_DLL_NAMEA));
#endif
#endif

		if (DllHandle)
		{
			AMFInit_Fn AmfInitFn = (AMFInit_Fn)FPlatformProcess::GetDllExport(DllHandle, TEXT(AMF_INIT_FUNCTION_NAME));

			if (AmfInitFn == nullptr)
			{
				return;
			}

			CHECK_AMF_RET(AmfInitFn(AMF_FULL_VERSION, &AmfFactory));

			bIsAvailable = true;
		}
	}

	bool FAmfCommon::InitializeContext(ERHIInterfaceType RHIType, const FString &RHIName, void *NativeDevice, void *NativeInstance, void *NativePhysicalDevice)
	{
		AMF_RESULT Res = AMF_FAIL;

		// Create context
		Res = AmfFactory->CreateContext(&AmfContext);

		if (RHIType == ERHIInterfaceType::D3D11)
		{
			Res = AmfContext->InitDX11(NativeDevice);
			UE_LOGF(LogEncoderAMF, Log, "Amf initialised with D3D11");
		}
		else if (RHIType == ERHIInterfaceType::D3D12)
		{
			Res = AMFContext2Ptr(AmfContext)->InitDX12(NativeDevice);
			UE_LOGF(LogEncoderAMF, Log, "Amf initialised with D3D12");
		}
		else if (RHIType == ERHIInterfaceType::Vulkan)
		{
			amf::AMFContext1Ptr pContext1(AmfContext);

			if (NativeDevice != NULL)
			{
				AmfVulkanDevice = {};
				AmfVulkanDevice.cbSizeof = sizeof(AMFVulkanDevice);
				AmfVulkanDevice.hInstance = (VkInstance)NativeInstance;
				AmfVulkanDevice.hPhysicalDevice = (VkPhysicalDevice)NativePhysicalDevice;
				AmfVulkanDevice.hDevice = (VkDevice)NativeDevice;

				Res = pContext1->InitVulkan(&AmfVulkanDevice);
			}
			else
			{
				Res = pContext1->InitVulkan(NativeDevice);
			}
			UE_LOGF(LogEncoderAMF, Log, "Amf initialised with Vulkan");
		}
		else
		{
			UE_LOGF(LogEncoderAMF, Fatal, "Currently %ls not supported by Amf as an RHI", *RHIName);
		}

		bIsCtxInitialized = (Res == AMF_OK);
		return bIsCtxInitialized;
	}
}
