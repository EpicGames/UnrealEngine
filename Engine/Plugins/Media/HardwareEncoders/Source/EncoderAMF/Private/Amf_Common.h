// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"

THIRD_PARTY_INCLUDES_START

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#pragma warning(push)
#pragma warning(disable : 4005)
#endif //PLATFORM_WINDOWS

#include <components/Component.h>
#include <core/Context.h>
#include <core/Platform.h>
#include "core/Interface.h"
#include "core/VulkanAMF.h"

#if PLATFORM_WINDOWS
#pragma warning(pop)
#include "Windows/HideWindowsPlatformTypes.h"
#endif //PLATFORM_WINDOWS

THIRD_PARTY_INCLUDES_END

DECLARE_LOG_CATEGORY_EXTERN(LogEncoderAMF, Log, All);

enum class ERHIInterfaceType;
namespace amf { class AMFFactory; }

namespace amf {
	struct AMFVulkanDevice;
}

#define CHECK_AMF_RET(AMF_call)\
{\
	AMF_RESULT Res = AMF_call;\
	if (!(Res== AMF_OK || Res==AMF_ALREADY_INITIALIZED))\
	{\
		UE_LOG(LogEncoderAMF, Error, TEXT("`" #AMF_call "` failed with error code: %d"), Res);\
		return;\
	}\
}

namespace AVEncoder
{
	using namespace amf;

    class FAmfCommon
    {
    public:
        // attempt to load Amf
        static FAmfCommon &Setup();
		
        // shutdown - release loaded dll
        static void Shutdown();

        bool GetIsAvailable() const { return bIsAvailable; }

		bool GetIsCtxInitialized() const { return bIsCtxInitialized; }
		
		bool CreateEncoder(amf::AMFComponentPtr& outEncoder);

		AMFContextPtr GetContext() { return AmfContext; }

        bool InitializeContext(ERHIInterfaceType RHIType, const FString& RHIName, void* NativeDevice, void* NativeInstance = nullptr, void* NativePhysicalDevice = nullptr);

        void DestroyContext()
        {
            if(bIsCtxInitialized)
            {
                AmfContext->Terminate();
				AmfContext = nullptr;
                bIsCtxInitialized = false;
            }
        }

    private:
        FAmfCommon() = default;

		void SetupAmfFunctions();

        static FCriticalSection ProtectSingleton;
        static FAmfCommon Singleton;

        amf_handle DllHandle = nullptr;
        AMFFactory* AmfFactory = NULL;
		AMFContextPtr AmfContext = NULL;
		AMFVulkanDevice AmfVulkanDevice;
        bool bIsAvailable = false;
        bool bWasSetUp = false;
        bool bIsCtxInitialized = false;
    };
}
