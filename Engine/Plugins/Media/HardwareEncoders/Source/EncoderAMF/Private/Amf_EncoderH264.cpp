// Copyright Epic Games, Inc. All Rights Reserved.

#include "Amf_EncoderH264.h"
#include "Amf_Common.h"
#include "CodecPacket.h"
#include "Misc/CommandLine.h"
#include "RHI.h"
#include "Async/Async.h"
#include "VideoEncoderFactory.h"
#include "VideoEncoderInputImpl.h"
#include <components/ComponentCaps.h>
#include <components/VideoEncoderVCE.h>
#include <core/Buffer.h>
#include <core/Data.h>
#include <core/PropertyStorage.h>

#define MAX_GPU_INDEXES 50
#define DEFAULT_BITRATE 1000000u
#define MAX_FRAMERATE_DIFF 0
#define MIN_UPDATE_FRAMERATE_SECS 15

#define AMF_VIDEO_ENCODER_START_TS L"StartTs"
#define AMF_BUFFER_INPUT_FRAME L"BufferInputFrame"

namespace
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	AMF_VIDEO_ENCODER_RATE_CONTROL_METHOD_ENUM ConvertRateControlModeAMF(AVEncoder::FVideoEncoder::RateControlMode mode)
	{
		switch (mode)
		{
			case AVEncoder::FVideoEncoder::RateControlMode::CONSTQP: return AMF_VIDEO_ENCODER_RATE_CONTROL_METHOD_CONSTANT_QP;
			case AVEncoder::FVideoEncoder::RateControlMode::VBR: return AMF_VIDEO_ENCODER_RATE_CONTROL_METHOD_LATENCY_CONSTRAINED_VBR;
			default:
			case AVEncoder::FVideoEncoder::RateControlMode::CBR: return AMF_VIDEO_ENCODER_RATE_CONTROL_METHOD_CBR;
		}
	}

	AMF_VIDEO_ENCODER_PROFILE_ENUM ConvertH264Profile(AVEncoder::FVideoEncoder::H264Profile profile)
	{
		switch (profile)
		{
			case AVEncoder::FVideoEncoder::H264Profile::CONSTRAINED_BASELINE: return AMF_VIDEO_ENCODER_PROFILE_CONSTRAINED_BASELINE;
			case AVEncoder::FVideoEncoder::H264Profile::BASELINE: return AMF_VIDEO_ENCODER_PROFILE_BASELINE;
			case AVEncoder::FVideoEncoder::H264Profile::MAIN: return AMF_VIDEO_ENCODER_PROFILE_MAIN;
			case AVEncoder::FVideoEncoder::H264Profile::CONSTRAINED_HIGH: return AMF_VIDEO_ENCODER_PROFILE_CONSTRAINED_HIGH;
			case AVEncoder::FVideoEncoder::H264Profile::HIGH: return AMF_VIDEO_ENCODER_PROFILE_HIGH;
			default:
			case AVEncoder::FVideoEncoder::H264Profile::AUTO: return AMF_VIDEO_ENCODER_PROFILE_BASELINE;
		}
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

namespace AVEncoder
{

	TAutoConsoleVariable<int32>  CVarAMFKeyframeInterval(
	TEXT("AMF.KeyframeInterval"),
	300,
	TEXT("Every N frames an IDR frame is sent. Default: 300. Note: A value <= 0 will disable sending of IDR frames on an interval."),
	ECVF_Default);

	template<typename T>
	void AMFCommandLineParseValue(const TCHAR* Match, TAutoConsoleVariable<T>& CVar)
	{
		T Value;
		if (FParse::Value(FCommandLine::Get(), Match, Value))
		{
			CVar->Set(Value, ECVF_SetByCommandline);
		}
	};

	void AMFCommandLineParseOption(const TCHAR* Match, TAutoConsoleVariable<bool>& CVar)
	{
		FString ValueMatch(Match);
		ValueMatch.Append(TEXT("="));
		FString Value;
		if (FParse::Value(FCommandLine::Get(), *ValueMatch, Value)) {
			if (Value.Equals(FString(TEXT("true")), ESearchCase::IgnoreCase)) {
				CVar->Set(true, ECVF_SetByCommandline);
			}
			else if (Value.Equals(FString(TEXT("false")), ESearchCase::IgnoreCase)) {
				CVar->Set(false, ECVF_SetByCommandline);
			}
		}
		else if (FParse::Param(FCommandLine::Get(), Match))
		{
			CVar->Set(true, ECVF_SetByCommandline);
		}
	}

	void AMFParseCommandLineFlags()
	{
		AsyncTask(ENamedThreads::GameThread, []()
		{ 
			AMFCommandLineParseValue(TEXT("-AMFKeyframeInterval="), CVarAMFKeyframeInterval);
		});
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	static bool GetEncoderInfo(FAmfCommon& Amf, FVideoEncoderInfo& EncoderInfo);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	bool FVideoEncoderAmf_H264::GetIsAvailable(FVideoEncoderInputImpl& InInput, FVideoEncoderInfo& OutEncoderInfo)
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		FAmfCommon& Amf = FAmfCommon::Setup();
		bool bIsAvailable = Amf.GetIsAvailable();
		if (bIsAvailable)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			OutEncoderInfo.CodecType = ECodecType::H264;
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
		
		return bIsAvailable;
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	void FVideoEncoderAmf_H264::Register(FVideoEncoderFactory& InFactory)
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		FAmfCommon& Amf = FAmfCommon::Setup();
		if (Amf.GetIsAvailable() && IsRHIDeviceAMD())
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			FVideoEncoderInfo	EncoderInfo;
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			if (GetEncoderInfo(Amf, EncoderInfo))
			{
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				InFactory.Register(EncoderInfo, []() {
					return TUniquePtr<FVideoEncoder>(new FVideoEncoderAmf_H264());
				});
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
			}
		}
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FVideoEncoderAmf_H264::FVideoEncoderAmf_H264()
		: Amf(FAmfCommon::Setup())
	{
		AMFParseCommandLineFlags();
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FVideoEncoderAmf_H264::~FVideoEncoderAmf_H264() { Shutdown(); }
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	bool FVideoEncoderAmf_H264::Setup(TSharedRef<FVideoEncoderInput> input, FLayerConfig const& config)
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		if (!Amf.GetIsAvailable())
		{
			UE_LOGF(LogEncoderAMF, Error, "Amf not avaliable.");
			return false;
		}
				
		TSharedRef<FVideoEncoderInputImpl>	Input(StaticCastSharedRef<FVideoEncoderInputImpl>(input));

		ERHIInterfaceType RHIType = ERHIInterfaceType::Hidden;

		// TODO fix initializing contexts
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FrameFormat = input->GetFrameFormat();
		switch (FrameFormat)
		{
#if PLATFORM_WINDOWS
		case AVEncoder::EVideoFrameFormat::D3D11_R8G8B8A8_UNORM:
			EncoderDevice = Input->GetD3D11EncoderDevice();
			RHIType = ERHIInterfaceType::D3D11;
			break;
		case AVEncoder::EVideoFrameFormat::D3D12_R8G8B8A8_UNORM:
			EncoderDevice = Input->GetD3D12EncoderDevice();
			RHIType = ERHIInterfaceType::D3D12;
			break;
#endif
		case AVEncoder::EVideoFrameFormat::VULKAN_R8G8B8A8_UNORM:
			EncoderDevice = Input->GetVulkanEncoderDevice();
			RHIType = ERHIInterfaceType::Vulkan;
			break;
		case AVEncoder::EVideoFrameFormat::Undefined:
		default:
			UE_LOGF(LogEncoderAMF, Error, "Frame format %ls is not currently supported by Amf Encoder on this platform.", *ToString(FrameFormat));
			return false;
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		if (!EncoderDevice)
		{
			UE_LOGF(LogEncoderAMF, Error, "Amf needs an encoder device.");
			return false;
		}
				
		if (!Amf.GetIsCtxInitialized())
		{	
			if (RHIType != ERHIInterfaceType::Vulkan)
			{
				if (!Amf.InitializeContext(RHIGetInterfaceType(), GDynamicRHI->GetName(), EncoderDevice))
				{
					UE_LOGF(LogEncoderAMF, Error, "Amf component not initialised");
				}
			}
			else
			{
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				FVulkanDataStruct* VulkanData = static_cast<FVulkanDataStruct*>(EncoderDevice);
				if (!Amf.InitializeContext(RHIGetInterfaceType(), GDynamicRHI->GetName(), VulkanData->VulkanDevice, VulkanData->VulkanInstance, VulkanData->VulkanPhysicalDevice))
				{
					UE_LOGF(LogEncoderAMF, Error, "Amf component not initialised");
				}
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
			}
		}

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FLayerConfig MutableConfig = config;
		if (MutableConfig.MaxFramerate == 0)
		{
			MutableConfig.MaxFramerate = 60;
		}

		return AddLayer(config);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FVideoEncoder::FLayer* FVideoEncoderAmf_H264::CreateLayer(uint32 layerIdx, FLayerConfig const& config)
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		auto const layer = new FAMFLayer(layerIdx, config, *this);
		if (!layer->Setup())
		{
			delete layer;
			return nullptr;
		}

		return layer;
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	void FVideoEncoderAmf_H264::DestroyLayer(FLayer* layer) { delete layer; }
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	void FVideoEncoderAmf_H264::Encode(const TSharedPtr<FVideoEncoderInputFrame> frame, FEncodeOptions const& options)
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		const TSharedPtr<FVideoEncoderInputFrameImpl> amfFrame = StaticCastSharedPtr<FVideoEncoderInputFrameImpl>(frame);
		for (auto& layer : Layers)
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		{
			FAMFLayer* amfLayer = static_cast<FAMFLayer*>(layer);
			AMF_RESULT Res = amfLayer->Encode(amfFrame, options);
			if(Res != AMF_OK)
			{
				UE_LOGF(LogEncoderAMF, Error, "AMF failed to encode frame.");
			}
		}
	}

	void FVideoEncoderAmf_H264::Flush()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		for (auto&& layer : Layers)
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		{
			auto const amfLayer = static_cast<FAMFLayer*>(layer);
			amfLayer->Flush();
		}
	}

	void FVideoEncoderAmf_H264::Shutdown()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		for (auto&& layer : Layers)
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		{
			auto const amfLayer = static_cast<FAMFLayer*>(layer);
			amfLayer->Shutdown();
			DestroyLayer(amfLayer);
		}

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Layers.Reset();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	

	// --- Amf_EncoderH264::FLayer ------------------------------------------------------------
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FVideoEncoderAmf_H264::FAMFLayer::FAMFLayer(uint32 layerIdx, FLayerConfig const& config, FVideoEncoderAmf_H264& encoder)
		: FLayer(config)
		, Encoder(encoder)
		, Amf(FAmfCommon::Setup())
		, LayerIndex(layerIdx)
	{
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FVideoEncoderAmf_H264::FAMFLayer::~FAMFLayer() {}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	bool FVideoEncoderAmf_H264::FAMFLayer::Setup()
	{
		return CreateSession() && CreateInitialConfig();
	}

	bool FVideoEncoderAmf_H264::FAMFLayer::CreateSession()
	{
		if (AmfEncoder == NULL)
		{
			return Amf.CreateEncoder(AmfEncoder) && AmfEncoder != NULL;
		}

		return AmfEncoder != NULL;
	}

	bool FVideoEncoderAmf_H264::FAMFLayer::CreateInitialConfig()
	{
		AMF_RESULT Result = AMF_OK;

		Result = AmfEncoder->SetProperty(AMF_VIDEO_ENCODER_USAGE, AMF_VIDEO_ENCODER_USAGE_LOW_LATENCY);

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		AMF_VIDEO_ENCODER_PROFILE_ENUM H264Profile = ConvertH264Profile(CurrentConfig.H264Profile);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		Result = AmfEncoder->SetProperty(AMF_VIDEO_ENCODER_PROFILE, H264Profile);
		Result = AmfEncoder->SetProperty(AMF_VIDEO_ENCODER_PROFILE_LEVEL, 51);

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		AMFRate frameRate = { CurrentConfig.MaxFramerate, 1 };
		Result = AmfEncoder->SetProperty(AMF_VIDEO_ENCODER_FRAMERATE, frameRate);
		CurrentFrameRate = CurrentConfig.MaxFramerate;

#if PLATFORM_WINDOWS
		Result = AmfEncoder->SetProperty(AMF_VIDEO_ENCODER_RATE_CONTROL_METHOD, ConvertRateControlModeAMF(CurrentConfig.RateControlMode));
		if (CurrentConfig.RateControlMode == RateControlMode::CBR)
		{
			AmfEncoder->SetProperty(AMF_VIDEO_ENCODER_FILLER_DATA_ENABLE, true);
		}
		Result = AmfEncoder->SetProperty(AMF_VIDEO_ENCODER_PEAK_BITRATE, CurrentConfig.MaxBitrate > -1 ? CurrentConfig.MaxBitrate : 10 * DEFAULT_BITRATE);
#endif
	
		Result = AmfEncoder->SetProperty(AMF_VIDEO_ENCODER_TARGET_BITRATE, CurrentConfig.TargetBitrate > -1 ? CurrentConfig.TargetBitrate : DEFAULT_BITRATE);

        Result = AmfEncoder->SetProperty(AMF_VIDEO_ENCODER_QUALITY_PRESET, AMF_VIDEO_ENCODER_QUALITY_PRESET_QUALITY);
		Result = AmfEncoder->SetProperty(AMF_VIDEO_ENCODER_B_PIC_PATTERN, 0);

		Result = AmfEncoder->SetProperty(AMF_VIDEO_ENCODER_MIN_QP, FMath::Clamp<amf_int64>(CurrentConfig.QPMin, 0, 51));
		Result = AmfEncoder->SetProperty(AMF_VIDEO_ENCODER_MAX_QP, CurrentConfig.QPMax > -1 ? FMath::Clamp<amf_int64>(CurrentConfig.QPMax, 0, 51) : 51);

		Result = AmfEncoder->SetProperty(AMF_VIDEO_ENCODER_QUERY_TIMEOUT, 16);

		int32 IdrPeriod = CVarAMFKeyframeInterval.GetValueOnAnyThread();
		if(IdrPeriod > 0)
		{
			Result = AmfEncoder->SetProperty(AMF_VIDEO_ENCODER_IDR_PERIOD, IdrPeriod);
		}

		Result = AmfEncoder->Init(AMF_SURFACE_BGRA, CurrentConfig.Width, CurrentConfig.Height);
		CurrentWidth = CurrentConfig.Width;
		CurrentHeight = CurrentConfig.Height;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		return Result == AMF_OK;
	}

	void FVideoEncoderAmf_H264::FAMFLayer::MaybeReconfigure()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FScopeLock lock(&ConfigMutex);
		if (NeedsReconfigure)
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		{	
			// Static properties - need ReInit
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			if (CurrentConfig.Width != CurrentWidth || CurrentConfig.Height != CurrentHeight || CurrentConfig.MaxFramerate != CurrentFrameRate)
			{
				AMF_RESULT Result = AMF_OK;

				AMFRate frameRate = { CurrentConfig.MaxFramerate, 1 };
				Result = AmfEncoder->SetProperty(AMF_VIDEO_ENCODER_FRAMERATE, frameRate);
				CurrentFrameRate = CurrentConfig.MaxFramerate;

				AmfEncoder->Flush();
				Result = AmfEncoder->ReInit(CurrentConfig.Width, CurrentConfig.Height);
				CurrentWidth = CurrentConfig.Width;
				CurrentHeight = CurrentConfig.Height;

				if (Result != AMF_OK)
				{
					UE_LOGF(LogEncoderAMF, Error, "Amf failed to ReInit for config change");
				}
			}
			PRAGMA_ENABLE_DEPRECATION_WARNINGS

			// Dynamic Properties
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			if (AmfEncoder->SetProperty(AMF_VIDEO_ENCODER_MIN_QP, FMath::Clamp<amf_int64>(CurrentConfig.QPMin, 0, 51)) != AMF_OK)
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			{
				UE_LOGF(LogEncoderAMF, Error, "Amf failed to set min qp");
			}

			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			if (AmfEncoder->SetProperty(AMF_VIDEO_ENCODER_MAX_QP, CurrentConfig.QPMax > -1 ? FMath::Clamp<amf_int64>(CurrentConfig.QPMax, 0, 51) : 51) != AMF_OK)
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			{
				UE_LOGF(LogEncoderAMF, Error, "Amf failed to set max qp");
			}

			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			if (AmfEncoder->SetProperty(AMF_VIDEO_ENCODER_TARGET_BITRATE, CurrentConfig.TargetBitrate) != AMF_OK)
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			{
				UE_LOGF(LogEncoderAMF, Error, "Amf failed to set target bitrate");
			}

#if PLATFORM_WINDOWS
			// Properties in this macro block are supposed to be dynamic but error when used with Vulkan
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			if (AmfEncoder->SetProperty(AMF_VIDEO_ENCODER_RATE_CONTROL_METHOD, ConvertRateControlModeAMF(CurrentConfig.RateControlMode)) != AMF_OK)
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			{
				UE_LOGF(LogEncoderAMF, Error, "Amf failed to set rate control method");
			}

			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			if (CurrentConfig.RateControlMode == RateControlMode::CBR)
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			{
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				if (AmfEncoder->SetProperty(AMF_VIDEO_ENCODER_FILLER_DATA_ENABLE, CurrentConfig.FillData) != AMF_OK)
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
				{
					UE_LOG(LogEncoderAMF, Error, TEXT("Amf failed to enable filler data to maintain CBR"));
				}
			}

			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			if (AmfEncoder->SetProperty(AMF_VIDEO_ENCODER_PEAK_BITRATE, CurrentConfig.MaxBitrate) != AMF_OK)
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			{
				UE_LOGF(LogEncoderAMF, Error, "Amf failed to set peak bitrate");
			}
#endif

			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			NeedsReconfigure = false;
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
		
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	AMF_RESULT FVideoEncoderAmf_H264::FAMFLayer::Encode(const TSharedPtr<FVideoEncoderInputFrameImpl> frame, FEncodeOptions const& options)
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		AMF_RESULT Result = AMF_FAIL;
		TSharedPtr<FInputOutput> Buffer = GetOrCreateSurface(frame);

		if (Buffer)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			if(CurrentConfig.Width != frame->GetWidth() || CurrentConfig.Height != frame->GetHeight())
			{
				CurrentConfig.Width = frame->GetWidth();
				CurrentConfig.Height = frame->GetHeight();
				NeedsReconfigure = true;
			}
			PRAGMA_ENABLE_DEPRECATION_WARNINGS

			
			MaybeReconfigure();

			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			Buffer->Surface->SetPts(frame->GetTimestampRTP());
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			amf_int64 Start_ts = FPlatformTime::Cycles64();
			Buffer->Surface->SetProperty(AMF_VIDEO_ENCODER_START_TS, Start_ts);
			Buffer->Surface->SetProperty(AMF_BUFFER_INPUT_FRAME, uintptr_t(frame.Get()));

#if PLATFORM_WINDOWS
			Buffer->Surface->SetProperty(AMF_VIDEO_ENCODER_STATISTICS_FEEDBACK, true);
#endif

			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			if (options.bForceKeyFrame)
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			{
				if (Buffer->Surface->SetProperty(AMF_VIDEO_ENCODER_FORCE_PICTURE_TYPE, AMF_VIDEO_ENCODER_PICTURE_TYPE_IDR) != AMF_OK)
				{
					UE_LOGF(LogEncoderAMF, Error, "Amf failed to force IDR picture type");
				}

				if (Buffer->Surface->SetProperty(AMF_VIDEO_ENCODER_INSERT_SPS, true) != AMF_OK)
				{				
					UE_LOGF(LogEncoderAMF, Error, "Amf failed to force SPS");
				}		
					
				if (Buffer->Surface->SetProperty(AMF_VIDEO_ENCODER_INSERT_PPS, true) != AMF_OK)
				{				
					UE_LOGF(LogEncoderAMF, Error, "Amf failed to force PPS");
				}
			}

			if (Buffer.IsValid())
			{
				Result = AmfEncoder->SubmitInput(Buffer->Surface);

				if (Result == AMF_NEED_MORE_INPUT)
				{
				}
				else if (Result != AMF_OK)
				{
					UE_LOGF(LogEncoderAMF, Error, "Amf submit error with %d", Result);
					// release input frame
					frame->Release();
				}
				else
				{
					// Note from Luke: Testing has shown this is okay to leave blocking on the calling thread.
					ProcessFrameBlocking();
				}
			}
		}

		return Result;
	}

	void FVideoEncoderAmf_H264::FAMFLayer::Flush()
	{
		AmfEncoder->Flush();
	}

	void FVideoEncoderAmf_H264::FAMFLayer::Shutdown()
	{
		Flush();
		CreatedSurfaces.Empty();

		if (AmfEncoder != NULL)
		{
			AmfEncoder->Terminate();
			AmfEncoder = NULL;
		}
	}

	void FVideoEncoderAmf_H264::FAMFLayer::ProcessFrameBlocking()
	{
		checkf(!bIsProcessingFrame, TEXT("There is already a frame being processing in the AMF encoder. Only one thing should call encode at a time."));

		bIsProcessingFrame = true;

		while(bIsProcessingFrame)
		{
			amf::AMFDataPtr data;
			AMF_RESULT Result = AmfEncoder->QueryOutput(&data);

			if (Result != AMF_OK)
			{
				// No frame ready yet in output.
				continue;
			}

			AMFBufferPtr OutBuffer(data);

			// Create packet with buffer contents
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			FCodecPacket Packet = FCodecPacket::Create(static_cast<const uint8*>(OutBuffer->GetNative()), OutBuffer->GetSize());
			PRAGMA_ENABLE_DEPRECATION_WARNINGS

			uint32 PictureType = AMF_VIDEO_ENCODER_PICTURE_TYPE_NONE;
			if (OutBuffer->GetProperty(AMF_VIDEO_ENCODER_OUTPUT_DATA_TYPE, &PictureType) != AMF_OK)
			{
				UE_LOGF(LogEncoderAMF, Error, "Amf failed to get picture type.");
			}
			else if (PictureType == AMF_VIDEO_ENCODER_OUTPUT_DATA_TYPE_IDR)
			{
				UE_LOGF(LogEncoderAMF, Verbose, "Generated IDR Frame");
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				Packet.IsKeyFrame = true;
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
			}
			

			if (RHIGetInterfaceType() != ERHIInterfaceType::Vulkan) // Amf with Vulkan doesn't currently support statistics
			{
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				if (OutBuffer->GetProperty(AMF_VIDEO_ENCODER_STATISTIC_FRAME_QP, &Packet.VideoQP) != AMF_OK)
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
				{
					UE_LOGF(LogEncoderAMF, Error, "Amf failed to get frame QP.");
				}
			}

			amf_int64 StartTs;
			if (OutBuffer->GetProperty(AMF_VIDEO_ENCODER_START_TS, &StartTs) != AMF_OK)
			{
				UE_LOGF(LogEncoderAMF, Error, "Amf failed to get encode start time.");
			}

			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			Packet.Timings.StartTs = FTimespan::FromSeconds(FPlatformTime::ToSeconds64(StartTs));
			Packet.Timings.FinishTs = FTimespan::FromSeconds(FPlatformTime::ToSeconds64(FPlatformTime::Cycles64()));
			Packet.Framerate = GetConfig().MaxFramerate;
			PRAGMA_ENABLE_DEPRECATION_WARNINGS

			TSharedPtr<FVideoEncoderInputFrameImpl> SourceFrame;
			if (OutBuffer->GetProperty(AMF_BUFFER_INPUT_FRAME, (intptr_t*)&SourceFrame) != AMF_OK)
			{
				UE_LOGF(LogEncoderAMF, Fatal, "Amf failed to get buffer input frame.");
			}

			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			if (Encoder.OnEncodedPacket)
			{
				Encoder.OnEncodedPacket(LayerIndex, SourceFrame, Packet);
			}
			PRAGMA_ENABLE_DEPRECATION_WARNINGS

			bIsProcessingFrame = false;
		}
		
	}

	template<class T>
	bool FVideoEncoderAmf_H264::FAMFLayer::GetCapability(const TCHAR* CapToQuery, T& OutCap) const
	{
		amf::AMFCapsPtr EncoderCaps;
		
		if (AmfEncoder->GetCaps(&EncoderCaps) != AMF_OK)
		{
			return false;
		}

		if (EncoderCaps->GetProperty(CapToQuery, &OutCap))
		{
			return false;
		}

		return true;
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	amf::AMFVulkanSurface* CreateVulkanSurface(VkImage Image, VkDeviceMemory DeviceMemory, EVideoFrameFormat Format, uint32 Size, uint32 Width, uint32 Height)
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		amf_int32 pixelFormat = 0;
		int pixelSize = 0;
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		switch (Format) 
		{
			default:
			case EVideoFrameFormat::VULKAN_R8G8B8A8_UNORM:
				pixelFormat = VK_FORMAT_B8G8R8A8_UNORM;
				break;
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		amf::AMFVulkanSurface* SurfaceTexture = new amf::AMFVulkanSurface();
		SurfaceTexture->cbSizeof = sizeof(amf::AMFVulkanSurface);
		SurfaceTexture->hImage = Image;
		SurfaceTexture->eUsage = amf::AMF_SURFACE_USAGE_DEFAULT;
		SurfaceTexture->hMemory = DeviceMemory;
		SurfaceTexture->iSize = Size;
		SurfaceTexture->eFormat = pixelFormat;
		SurfaceTexture->iWidth = Width;
		SurfaceTexture->iHeight = Height;

		SurfaceTexture->Sync.cbSizeof = sizeof(SurfaceTexture->Sync);
		SurfaceTexture->Sync.hSemaphore = VK_NULL_HANDLE;
		SurfaceTexture->Sync.bSubmitted = false;

		SurfaceTexture->eCurrentLayout = VK_IMAGE_LAYOUT_PRESENT_SRC_KHR;

		return SurfaceTexture;
	}

	TSharedPtr<FVideoEncoderAmf_H264::FAMFLayer::FInputOutput> FVideoEncoderAmf_H264::FAMFLayer::GetOrCreateSurface(const TSharedPtr<FVideoEncoderInputFrameImpl> InFrame)
	{
		void* TextureToCompress = nullptr;

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		switch (InFrame->GetFormat())
		{
#if PLATFORM_WINDOWS
		case EVideoFrameFormat::D3D11_R8G8B8A8_UNORM:
			TextureToCompress = InFrame->GetD3D11().EncoderTexture;
			break;
		case EVideoFrameFormat::D3D12_R8G8B8A8_UNORM:
			TextureToCompress = InFrame->GetD3D12().EncoderTexture;
			break;
#endif
		case EVideoFrameFormat::VULKAN_R8G8B8A8_UNORM:
		{
			const FVideoEncoderInputFrame::FVulkan& Frame = InFrame->GetVulkan();
			if (!Frame.EncoderSurface)
			{
				Frame.EncoderSurface = CreateVulkanSurface(Frame.EncoderTexture,
					Frame.EncoderDeviceMemory,
					InFrame->GetFormat(),
					Frame.EncoderMemorySize,
					InFrame->GetWidth(),
					InFrame->GetHeight()
				);				
			}
	
			//TODO there seems to be some some concurrency issues under Windows might be that we are not adding semaphores
			// will revisit later 	
			// VulkanRHI::FSemaphore* WaitSemaphore = new VulkanRHI::FSemaphore(*(GVulkanRHI->GetDevice()));
			// static_cast<amf::AMFVulkanSurface*>(Frame.EncoderSurface)->Sync.hSemaphore = WaitSemaphore->GetHandle();
			// static_cast<amf::AMFVulkanSurface*>(Frame.EncoderSurface)->Sync.bSubmitted = true;

			// ENQUEUE_RENDER_COMMAND(FAddAmfSemaphore)([WaitSemaphore, Device=GVulkanRHI->GetDevice()](FRHICommandListImmediate& RHICmdList)
			// {
			// 	FVulkanCommandBufferManager* CmdBufferMgr = Device->GetImmediateContext().GetCommandBufferManager();
			// 	FVulkanCmdBuffer* CmdBuffer = CmdBufferMgr->GetUploadCmdBuffer();
			// 	CmdBuffer->AddWaitSemaphore(VK_PIPELINE_STAGE_COLOR_ATTACHMENT_OUTPUT_BIT, WaitSemaphore);
			// });

			TextureToCompress = Frame.EncoderSurface;
			InFrame->OnReleaseVulkanSurface = [](void* Surface){ delete (amf::AMFVulkanSurface*) Surface; };
			break;
		}
		case EVideoFrameFormat::Undefined:
		default:
			UE_LOGF(LogEncoderAMF, Error, "Video Frame Format %ls not supported by Amf on this platform.", *ToString(InFrame->GetFormat()));
			break;
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		if (!TextureToCompress)
		{
			UE_LOGF(LogEncoderAMF, Fatal, "Got passed a null pointer.");
				return nullptr;
		}

		// Check if texture already has a buffer surface
		TSharedPtr<FInputOutput> Buffer = nullptr;
		for (TSharedPtr<FInputOutput> SearchBuffer : CreatedSurfaces)
		{
			if (SearchBuffer->TextureToCompress == TextureToCompress)
			{
				Buffer = SearchBuffer;
				break;
			}
		}

		// if texture does not already have a buffer surface create one
		if (!Buffer)
		{
			if (!CreateSurface(Buffer, InFrame, TextureToCompress))
			{
				InFrame->Release();
				UE_LOGF(LogEncoderAMF, Error, "Amf failed to create buffer.");
			}
			else
			{
				CreatedSurfaces.Push(Buffer);
			}
		}

		return Buffer;
	}

	class FSampleObserver : public AMFSurfaceObserver
	{
	public:
		FSampleObserver(const TSharedPtr<FVideoEncoderInputFrameImpl> Frame) : SourceFrame(Frame) {}
		virtual ~FSampleObserver() {}

	protected:
		virtual void AMF_STD_CALL OnSurfaceDataRelease(AMFSurface *pSurface)
		{
			SourceFrame->Release();
			delete this;
		}

	private:
		const TSharedPtr<FVideoEncoderInputFrameImpl> SourceFrame;
	};

	bool FVideoEncoderAmf_H264::FAMFLayer::CreateSurface(TSharedPtr<FVideoEncoderAmf_H264::FAMFLayer::FInputOutput>& OutBuffer, const TSharedPtr<FVideoEncoderInputFrameImpl> SourceFrame, void* TextureToCompress)
	{
		AMF_RESULT Result = AMF_OK;

		OutBuffer = MakeShared<FInputOutput>();

		if (TextureToCompress)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			switch (SourceFrame->GetFormat())
			{
#if PLATFORM_WINDOWS
			case EVideoFrameFormat::D3D11_R8G8B8A8_UNORM:
				Result = Amf.GetContext()->CreateSurfaceFromDX11Native(TextureToCompress, &(OutBuffer->Surface), new FSampleObserver(SourceFrame));
				break;
			case EVideoFrameFormat::D3D12_R8G8B8A8_UNORM:
				Result = amf::AMFContext2Ptr(Amf.GetContext())->CreateSurfaceFromDX12Native(TextureToCompress, &(OutBuffer->Surface), new FSampleObserver(SourceFrame));
				break;
#endif
			case EVideoFrameFormat::VULKAN_R8G8B8A8_UNORM:
				Result = amf::AMFContext2Ptr(Amf.GetContext())->CreateSurfaceFromVulkanNative(TextureToCompress, &(OutBuffer->Surface), new FSampleObserver(SourceFrame));
				break;
			case EVideoFrameFormat::Undefined:
			default:
				UE_LOGF(LogEncoderAMF, Error, "Video format %ls not inplemented for Amf on this platform", *ToString(SourceFrame->GetFormat()));
				break;
			}
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
		else
		{
			UE_LOGF(LogEncoderAMF, Error, "Amf received nullptr to texture.");
			return false;
		}

		return Result == AMF_OK;
	}

	static bool GetEncoderSupportedProfiles(AMFCapsPtr EncoderCaps, uint32& OutSupportedProfiles)
	{
		uint32 maxProfile;
		if (EncoderCaps->GetProperty(AMF_VIDEO_ENCODER_CAP_MAX_PROFILE, &maxProfile) != AMF_OK)
		{
			return false;
		}

		if (maxProfile >= AMF_VIDEO_ENCODER_PROFILE_BASELINE)
		{
			OutSupportedProfiles |= H264Profile_Baseline;
		}

		if (maxProfile >= AMF_VIDEO_ENCODER_PROFILE_MAIN)
		{
			OutSupportedProfiles |= H264Profile_Main;
		}

		if (maxProfile >= AMF_VIDEO_ENCODER_PROFILE_HIGH)
		{
			OutSupportedProfiles |= H264Profile_High;
		}

		if (maxProfile >= AMF_VIDEO_ENCODER_PROFILE_CONSTRAINED_BASELINE)
		{
			OutSupportedProfiles |= H264Profile_ConstrainedBaseline;
		}

		if (maxProfile >= AMF_VIDEO_ENCODER_PROFILE_CONSTRAINED_HIGH)
		{
			OutSupportedProfiles |= H264Profile_ConstrainedHigh;
		}

		return true;
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	static bool GetEncoderSupportedInputFormats(AMFIOCapsPtr IOCaps, TArray<EVideoFrameFormat>& OutSupportedInputFormats)
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		// TODO check if we actually need to query Amf for this
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
#if PLATFORM_WINDOWS
		OutSupportedInputFormats.Push(EVideoFrameFormat::D3D11_R8G8B8A8_UNORM);
		OutSupportedInputFormats.Push(EVideoFrameFormat::D3D12_R8G8B8A8_UNORM);
#endif
		OutSupportedInputFormats.Push(EVideoFrameFormat::VULKAN_R8G8B8A8_UNORM);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		return true;
	}


	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	static bool GetEncoderInfo(FAmfCommon& AMF, FVideoEncoderInfo& EncoderInfo)
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		bool bSuccess = true;

		AMF.InitializeContext(RHIGetInterfaceType(), GDynamicRHI->GetName(), NULL);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		EncoderInfo.CodecType = ECodecType::H264;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		
		// Create temp component
		AMFComponentPtr TempEncoder;
		bool bCreatedEncoder = AMF.CreateEncoder(TempEncoder);

		if(!bCreatedEncoder || TempEncoder == NULL)
		{
			UE_LOGF(LogEncoderAMF, Warning, "Failed to created AMF encoder on AMD hardware. Consider trying a different driver version.");
			return false;
		}

		AMFCapsPtr EncoderCaps;
		TempEncoder->GetCaps(&EncoderCaps);

		AMFIOCapsPtr InputCaps;
		EncoderCaps->GetInputCaps(&InputCaps);

		uint32 LevelMax;
		if (EncoderCaps->GetProperty(AMF_VIDEO_ENCODER_CAP_MAX_LEVEL, &LevelMax) == AMF_OK)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			EncoderInfo.H264.MinLevel = 9;														// Like NVENC we hard min at 9
			EncoderInfo.H264.MaxLevel = (LevelMax < 9) ? 9 : (LevelMax > 52) ? 52 : LevelMax;	// Like NVENC we hard max at 52
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
		else
		{
			UE_LOGF(LogEncoderAMF, Error, "Failed to query min/max h264 level supported by Amf (reported max=%d).", LevelMax);
			bSuccess = false;
		}

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (!GetEncoderSupportedProfiles(EncoderCaps, EncoderInfo.H264.SupportedProfiles) ||
			!GetEncoderSupportedInputFormats(InputCaps, EncoderInfo.SupportedInputFormats))
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		{
			bSuccess = false;
		}

		TempEncoder->Terminate();
		TempEncoder = nullptr;

		AMF.DestroyContext();

		return bSuccess;
	}

} /* namespace AVEncoder */

#undef MIN_UPDATE_FRAMERATE_SECS
