// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once



#include "HAL/ThreadSafeBool.h"
#include "Misc/DateTime.h"
#include "VideoEncoder.h"
#include "VideoEncoderInput.h"
#include <components/Component.h>
#include <core/Interface.h>
#include <core/Platform.h>
#include <core/Result.h>
#include <core/Surface.h>
#include <core/Variant.h>

namespace AVEncoder { class FAmfCommon; }
namespace AVEncoder { class FVideoEncoderFactory; }
namespace AVEncoder { class FVideoEncoderInputFrameImpl; }
namespace AVEncoder { class FVideoEncoderInputImpl; }

namespace AVEncoder
{
	using namespace amf;

    PRAGMA_DISABLE_DEPRECATION_WARNINGS
	class FVideoEncoderAmf_H264 : public FVideoEncoder
    PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
    public:
        virtual ~FVideoEncoderAmf_H264() override;

        // query whether or not encoder is supported and available
        PRAGMA_DISABLE_DEPRECATION_WARNINGS
		static bool GetIsAvailable(FVideoEncoderInputImpl &InInput, FVideoEncoderInfo &OutEncoderInfo);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

        // register encoder with video encoder factory
        static void Register(FVideoEncoderFactory &InFactory);

        PRAGMA_DISABLE_DEPRECATION_WARNINGS
		bool Setup(TSharedRef<FVideoEncoderInput> input, FLayerConfig const& config) override;
		void Encode(const TSharedPtr<FVideoEncoderInputFrame> frame, FEncodeOptions const& options) override;
        PRAGMA_ENABLE_DEPRECATION_WARNINGS
        void Flush();
        void Shutdown() override;

    protected:
        PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FLayer* CreateLayer(uint32 layerIdx, FLayerConfig const& config) override;
		void DestroyLayer(FLayer *layer) override;
        PRAGMA_ENABLE_DEPRECATION_WARNINGS

    private:
        FVideoEncoderAmf_H264();

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		class FAMFLayer : public FLayer
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		{
		public:

			class FInputOutput
			{
				public:
					void* TextureToCompress;
					AMFSurfacePtr Surface;
			};

			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			FAMFLayer(uint32 layerIdx, FLayerConfig const& config, FVideoEncoderAmf_H264& encoder);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			~FAMFLayer();

			bool Setup();
			bool CreateSession();
			bool CreateInitialConfig();
			void UpdateConfig();

			template<class T> 
			bool GetProperty(const TCHAR* PropertyToQuery, T& outProperty, const T& (*func)(const AMFVariantStruct*)) const;

			template<class T>
			bool GetCapability(const TCHAR* CapToQuery, T& OutCap) const;

			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			AMF_RESULT Encode(const TSharedPtr<FVideoEncoderInputFrameImpl> frame, FEncodeOptions const& options);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			void Flush();
			void Shutdown();
			void UpdateBitrate(uint32 InMaxBitRate, uint32 InTargetBitRate);
			void UpdateResolution(uint32 InMaxBitRate, uint32 InTargetBitRate);
			void MaybeReconfigure();
			void ProcessFrameBlocking();
			TSharedPtr<FInputOutput> GetOrCreateSurface(const TSharedPtr<FVideoEncoderInputFrameImpl> InFrame);
			bool CreateSurface(TSharedPtr<FInputOutput>& OutBuffer, const TSharedPtr<FVideoEncoderInputFrameImpl> SourceFrame, void* TextureToCompress);

			FVideoEncoderAmf_H264& Encoder;
			FAmfCommon& Amf;
			uint32 LayerIndex;
			AMFComponentPtr AmfEncoder = NULL;
			FDateTime LastKeyFrameTime = 0;
			bool bForceNextKeyframe = false;
			TArray<TSharedPtr<FInputOutput>> CreatedSurfaces;
			FThreadSafeBool bUpdateConfig = false;
			uint32 CurrentWidth;
			uint32 CurrentHeight;
			uint32 CurrentFrameRate;
			FThreadSafeBool bIsProcessingFrame = false;
		};

		FAmfCommon& Amf;
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		EVideoFrameFormat FrameFormat = EVideoFrameFormat::Undefined;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		void* EncoderDevice;

		uint32 MaxFramerate = 0;
		amf_int64 MinQP = -1;
		RateControlMode RateMode = RateControlMode::CBR;
		bool FillData = false;
	};
}
