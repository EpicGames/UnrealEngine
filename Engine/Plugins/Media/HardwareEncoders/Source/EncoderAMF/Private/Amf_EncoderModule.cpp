// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"

#include "IVulkanDynamicRHI.h"

#include "Amf_Common.h"
#include "Amf_EncoderH264.h"
#include "Misc/App.h"
#include "VideoEncoderFactory.h"

#include "Misc/CoreDelegates.h"
#include "RHIDefinitions.h"

class FAMFEncoderModule : public IModuleInterface
{
public:
	void StartupModule()
	{
		using namespace AVEncoder;
		if(FApp::CanEverRender())
		{
			FAmfCommon& AMF = FAmfCommon::Setup();

			if(AMF.GetIsAvailable())
			{
#if PLATFORM_WINDOWS
				const TCHAR* DynamicRHIModuleName = GetSelectedDynamicRHIModuleName(false);
#elif PLATFORM_LINUX
				const TCHAR* DynamicRHIModuleName = TEXT("VulkanRHI");
#endif
				if(FString("VulkanRHI") == FString(DynamicRHIModuleName))
				{
#if PLATFORM_WINDOWS || PLATFORM_LINUX
					UE_LOGF(LogEncoderAMF, Error, "Vulkan AMF is currently unsuported and has been disabled.");
					return;
#else
					AMF.InitializeContext(ERHIInterfaceType::Vulkan, "Vulkan", NULL);
					amf::AMFContext1Ptr pContext1(AMF.GetContext());

					amf_size NumDeviceExtensions = 0;
					pContext1->GetVulkanDeviceExtensions(&NumDeviceExtensions, nullptr);

					TArray<const ANSICHAR*> ExtentionsToAdd; 
					ExtentionsToAdd.AddUninitialized(5);

					pContext1->GetVulkanDeviceExtensions(&NumDeviceExtensions, ExtentionsToAdd.GetData());

					AMF.DestroyContext();

					IVulkanDynamicRHI::AddEnabledDeviceExtensionsAndLayers(ExtentionsToAdd, TArray<const ANSICHAR*>());
#endif
				}

				FCoreDelegates::GetOnPostEngineInit().AddLambda([]()
				{
					if (IsRHIDeviceAMD())
					{
						PRAGMA_DISABLE_DEPRECATION_WARNINGS
						FVideoEncoderAmf_H264::Register(FVideoEncoderFactory::Get());
						PRAGMA_ENABLE_DEPRECATION_WARNINGS
					}
					
				});
				
				AMFStarted = true;
			}
		}
	}

	void ShutdownModule()
	{
		using namespace AVEncoder;
		if(AMFStarted)
		{
			FAmfCommon& AMF = FAmfCommon::Setup();
			AMF.Shutdown();
			AMFStarted = false;
		}
	}

private:
	bool AMFStarted = false;
};

IMPLEMENT_MODULE(FAMFEncoderModule, EncoderAMF);
