// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;
using System;

public class EncoderNVENC: ModuleRules
{
	public EncoderNVENC(ReadOnlyTargetRules Target) : base(Target)
	{
		// Without these two compilation fails on VS2017 with D8049: command line is too long to fit in debug record.
		bLegacyPublicIncludePaths = false;
		DefaultBuildSettings = BuildSettingsVersion.Latest;

		PrivateDependencyModuleNames.AddRange(new string[] {
			"Engine",
			"AVEncoder",
			"VulkanRHI",
			"nvEncode",
			"CUDA"
		});

		PublicDependencyModuleNames.AddRange(new string[] {
			"RenderCore",
			"Core",
			"RHI"
		});

		string EngineSourceDirectory = Path.GetFullPath(Target.RelativeEnginePath);

		PrivateIncludePaths.Add(Path.Combine(EngineSourceDirectory, "Source/Runtime/AVEncoder/Private"));

		AddEngineThirdPartyPrivateStaticDependencies(Target, "Vulkan");
		if (Target.IsInPlatformGroup(UnrealPlatformGroup.Windows))
		{
			AddEngineThirdPartyPrivateStaticDependencies(Target, "DX11");
		}
	}
}
