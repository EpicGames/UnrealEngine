// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NVENC_Common.h"
#include "Engine/Engine.h"

// Stats about NVENC that can displayed either in the in-application HUD or in the log.
class FNVENCStats : FTickableGameObject
{
	public:

        FNVENCStats()
            : bOutputToLog(false)
            , bOutputToScreen(false)
        {

        }

		static FNVENCStats& Get()
        {
            static FNVENCStats Stats;
	        return Stats;
        }

        void Tick(float DeltaTime)
        {
            if (!GEngine)
            {
                return;
            }

            if(!bOutputToScreen && !bOutputToLog)
            {
                return;
            }

            int Id = 99;
            EmitStat(++Id, FString::Printf(TEXT("MaybeReconfigure(): %.5f ms"), ReconfigureMs.Get()));
            EmitStat(++Id, FString::Printf(TEXT("UnlockOutputBuffer(): %.5f ms"), UnlockOutputBufferMs.Get()));
            EmitStat(++Id, FString::Printf(TEXT("LockOutputBuffer(): %.5f ms"), LockOutputBufferMs.Get()));
            EmitStat(++Id, FString::Printf(TEXT("nvEncEncodePicture(): %.5f ms"), nvEncEncodePictureMs.Get()));
            EmitStat(++Id, FString::Printf(TEXT("ProcessFramesFunc(): %.5f ms"), ProcessFramesFuncMs.Get()));
            EmitStat(++Id, FString::Printf(TEXT("QueueEncode(): %.5f ms"), QueueEncodeMs.Get()));
            EmitStat(++Id, FString::Printf(TEXT("Total encoder latency: %.5f ms"), TotalEncoderLatencyMs.Get()));
            EmitStat(++Id, "------------ NVENC Stats ------------");
        }

        FORCEINLINE TStatId GetStatId() const { RETURN_QUICK_DECLARE_CYCLE_STAT(FNVENCStats, STATGROUP_Tickables); }

        void SetOutputToScreen(bool bInOutputToScreen)
        {
            bOutputToScreen = bInOutputToScreen;
            GAreScreenMessagesEnabled = bInOutputToScreen;
        }

        void SetOutputToLog(bool bInOutputToLog)
        {
            bOutputToLog = bInOutputToLog;
        }

		void SetProcessFramesFuncLatency(double InProcessFramesFuncMs)
        {
            this->ProcessFramesFuncMs.Update(InProcessFramesFuncMs);
        }

        void SetTotalEncoderLatency(double InTotalEncoderLatencyMs)
        {
            this->TotalEncoderLatencyMs.Update(InTotalEncoderLatencyMs);
        }

        void SetnvEncEncodePictureLatency(double InnvEncEncodePictureMs)
        {
            this->nvEncEncodePictureMs.Update(InnvEncEncodePictureMs);
        }

        void SetReconfigureLatency(double InReconfigureMs)
        {
            this->ReconfigureMs.Update(InReconfigureMs);
        }

        void SetQueueEncodeLatency(double InQueueEncodeMs)
        {
            this->QueueEncodeMs.Update(InQueueEncodeMs);
        }

        void SetLockOutputBufferLatency(double InLockOutputBufferMs)
        {
            this->LockOutputBufferMs.Update(InLockOutputBufferMs);
        }

        void SetUnlockOutputBufferLatency(double InUnlockOutputBufferMs)
        {
            this->UnlockOutputBufferMs.Update(InUnlockOutputBufferMs);
        }

	private:
		void EmitStat(int UniqueId, FString StringToEmit)
        {
            GEngine->AddOnScreenDebugMessage(UniqueId, 0, FColor::Green, StringToEmit, false /* newer on top */);

            if(this->bOutputToLog)
            {
                UE_LOGF(LogEncoderNVENC, Log, "%ls", *StringToEmit);
            }
        }

    private:

        template<uint32 SmoothingPeriod>
        class FSmoothedValue
        {
        public:
            double Get() const
            {
                return Value;
            }

            void Update(double NewValue)
            {
                Value = Value * (SmoothingPeriod - 1) / SmoothingPeriod + NewValue / SmoothingPeriod;
            }

            void Reset()
            {
                Value = 0;
            }

        private:
            TAtomic<double> Value{ 0 };
        };

	private:

        static constexpr uint32 SmoothingPeriod = 3 * 60; 
        bool bOutputToLog;
        bool bOutputToScreen;

		FSmoothedValue<SmoothingPeriod> ProcessFramesFuncMs;
        FSmoothedValue<SmoothingPeriod> TotalEncoderLatencyMs;
        FSmoothedValue<SmoothingPeriod> nvEncEncodePictureMs;
        FSmoothedValue<SmoothingPeriod> QueueEncodeMs;
        FSmoothedValue<SmoothingPeriod> ReconfigureMs;
        FSmoothedValue<SmoothingPeriod> LockOutputBufferMs;
        FSmoothedValue<SmoothingPeriod> UnlockOutputBufferMs;
};
