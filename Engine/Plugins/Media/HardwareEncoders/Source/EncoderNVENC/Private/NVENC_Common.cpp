// Copyright Epic Games, Inc. All Rights Reserved.

#include "NVENC_Common.h"
#include "CoreGlobals.h"
#include "HAL/PlatformProcess.h"
#include "Misc/ScopeLock.h"

DEFINE_LOG_CATEGORY(LogEncoderNVENC);

// define a function pointer for creating an instance of nvEncodeAPI
typedef NVENCSTATUS(NVENCAPI* NVENCAPIPROC)(NV_ENCODE_API_FUNCTION_LIST*);

namespace AVEncoder
{
	FCriticalSection FNVENCCommon::ProtectSingleton;
	FNVENCCommon FNVENCCommon::Singleton;

	// attempt to load NVENC
	FNVENCCommon& FNVENCCommon::Setup()
	{
		FScopeLock Guard(&ProtectSingleton);
		if(!Singleton.bWasSetUp)
		{
			Singleton.bWasSetUp = true;
			Singleton.SetupNVENCFunctions();
		}
		return Singleton;
	}

	// shutdown - release loaded dll
	void FNVENCCommon::Shutdown()
	{
		FScopeLock Guard(&ProtectSingleton);
		if(Singleton.bWasSetUp)
		{
			Singleton.bWasSetUp = false;
			Singleton.bIsAvailable = false;
			if(Singleton.DllHandle)
			{
				FPlatformProcess::FreeDllHandle(Singleton.DllHandle);
				Singleton.DllHandle = nullptr;
			}
		}
	}

	void FNVENCCommon::SetupNVENCFunctions()
	{
		check(!bIsAvailable);

#if PLATFORM_WINDOWS
		// Early out on non-supported windows versions
		if (!FPlatformMisc::VerifyWindowsVersion(6, 2))
		{
			return;
		}
#endif

		// clear function call table
		FMemory::Memzero(static_cast<NV_ENCODE_API_FUNCTION_LIST*>(this), sizeof(NV_ENCODE_API_FUNCTION_LIST));

		// name of DLL/SO library
#if PLATFORM_WINDOWS
#if defined _WIN64
		const TCHAR* DllName = TEXT("nvEncodeAPI64.dll");
#else
		const TCHAR* DllName = TEXT("nvEncodeAPI.dll");
#endif // _WIN64
#elif PLATFORM_LINUX
		const TCHAR* DllName = TEXT("libnvidia-encode.so.1");
#else
		const TCHAR* DllName = nullptr;
#endif
		if(DllName)
		{
			DllHandle = FPlatformProcess::GetDllHandle(DllName);
			if(DllHandle)
			{
				NVENCAPIPROC NvEncodeAPICreateInstanceFunc = (NVENCAPIPROC)(FPlatformProcess::GetDllExport(DllHandle, TEXT("NvEncodeAPICreateInstance")));

				checkf(NvEncodeAPICreateInstanceFunc != nullptr, TEXT("NvEncodeAPICreateInstance failed"));

				if(NvEncodeAPICreateInstanceFunc)
				{
					version = NV_ENCODE_API_FUNCTION_LIST_VER;
					NVENCSTATUS Result = NvEncodeAPICreateInstanceFunc(this);
					if (ensureMsgf(Result == NV_ENC_SUCCESS, TEXT("Unable to create NvEnc API function list: error %d"), Result))
					{
						bIsAvailable = true;
					}
				}
			}
		}
	}
} // namespace AVEncoder
