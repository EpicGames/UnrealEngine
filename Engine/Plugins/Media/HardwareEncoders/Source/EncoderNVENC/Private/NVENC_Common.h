// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"

THIRD_PARTY_INCLUDES_START

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#endif

#include <nvEncodeAPI.h>

#if PLATFORM_WINDOWS
#include "Windows/HideWindowsPlatformTypes.h"
#endif

THIRD_PARTY_INCLUDES_END

DECLARE_LOG_CATEGORY_EXTERN(LogEncoderNVENC, Log, All);


// helper macro to define and clear NVENC structures
//  also sets the struct version number
#define NVENCStruct(OfType, VarName) \
	OfType	VarName;	\
	FMemory::Memzero(VarName);	\
	VarName.version = OfType ## _VER


namespace AVEncoder
{
	class FNVENCCommon : public NV_ENCODE_API_FUNCTION_LIST
	{
	public:
		// attempt to load NVENC 
		static FNVENCCommon& Setup();
		// shutdown - release loaded dll
		static void Shutdown();

		bool GetIsAvailable() const { return bIsAvailable; }

		FString GetErrorString(void* Encoder, NVENCSTATUS ForStatus)
		{
			const char* LastError = nvEncGetLastErrorString(Encoder);
			if (LastError)
			{
				return FString::Format(TEXT("error {0} ('{1}')"), { ForStatus, UTF8_TO_TCHAR(LastError) });
			}

			return FString("Error getting error string! Is NVENC configured correctly?");
		}

	private:
		FNVENCCommon() = default;

		void SetupNVENCFunctions();

		static FCriticalSection			ProtectSingleton;
		static FNVENCCommon				Singleton;
		void* DllHandle = nullptr;
		bool							bIsAvailable = false;
		bool							bWasSetUp = false;
	};

} /* namespace AVEncoder */
