// Copyright Epic Games, Inc. All Rights Reserved.

#include "NVENC_EncoderH264.h"
#include "CodecPacket.h"
#include "Misc/CommandLine.h"
#include "RHI.h"
#include "CudaModule.h"
#include "Async/Async.h"
#include "NVENCStats.h"
#include "VideoEncoderFactory.h"
#include "VideoEncoderInputImpl.h"

#define MAX_GPU_INDEXES 50
#define DEFAULT_BITRATE 1000000u
#define MAX_FRAMERATE_DIFF 0
#define MIN_UPDATE_FRAMERATE_SECS 5

namespace
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	NV_ENC_PARAMS_RC_MODE ConvertRateControlModeNVENC(AVEncoder::FVideoEncoder::RateControlMode mode)
	{
		switch (mode)
		{
			case AVEncoder::FVideoEncoder::RateControlMode::CONSTQP:
				return NV_ENC_PARAMS_RC_CONSTQP;
			case AVEncoder::FVideoEncoder::RateControlMode::VBR:
				return NV_ENC_PARAMS_RC_VBR;
			default:
			case AVEncoder::FVideoEncoder::RateControlMode::CBR:
				return NV_ENC_PARAMS_RC_CBR;
		}
	}

	NV_ENC_MULTI_PASS ConvertMultipassModeNVENC(AVEncoder::FVideoEncoder::MultipassMode mode)
	{
		switch (mode)
		{
			case AVEncoder::FVideoEncoder::MultipassMode::DISABLED:
				return NV_ENC_MULTI_PASS_DISABLED;
			case AVEncoder::FVideoEncoder::MultipassMode::QUARTER:
				return NV_ENC_TWO_PASS_QUARTER_RESOLUTION;
			default:
			case AVEncoder::FVideoEncoder::MultipassMode::FULL:
				return NV_ENC_TWO_PASS_FULL_RESOLUTION;
		}
	}

	GUID ConvertH264Profile(AVEncoder::FVideoEncoder::H264Profile profile)
	{
		switch (profile)
		{
			default:
			case AVEncoder::FVideoEncoder::H264Profile::AUTO:
				return NV_ENC_CODEC_PROFILE_AUTOSELECT_GUID;
			case AVEncoder::FVideoEncoder::H264Profile::BASELINE:
				return NV_ENC_H264_PROFILE_BASELINE_GUID;
			case AVEncoder::FVideoEncoder::H264Profile::MAIN:
				return NV_ENC_H264_PROFILE_MAIN_GUID;
			case AVEncoder::FVideoEncoder::H264Profile::HIGH:
				return NV_ENC_H264_PROFILE_HIGH_GUID;
			case AVEncoder::FVideoEncoder::H264Profile::HIGH444:
				return NV_ENC_H264_PROFILE_HIGH_444_GUID;
			case AVEncoder::FVideoEncoder::H264Profile::STEREO:
				return NV_ENC_H264_PROFILE_STEREO_GUID;
			case AVEncoder::FVideoEncoder::H264Profile::PROGRESSIVE_HIGH:
				return NV_ENC_H264_PROFILE_PROGRESSIVE_HIGH_GUID;
			case AVEncoder::FVideoEncoder::H264Profile::CONSTRAINED_HIGH:
				return NV_ENC_H264_PROFILE_CONSTRAINED_HIGH_GUID;
		}
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
} // namespace

namespace AVEncoder
{
	// Console variables for NVENC

	TAutoConsoleVariable<int32> CVarNVENCIntraRefreshPeriodFrames(
		TEXT("NVENC.IntraRefreshPeriodFrames"),
		0,
		TEXT("The total number of frames between each intra refresh. Smallers values will cause intra refresh more often. Default: 0. Values <= 0 will disable intra refresh."),
		ECVF_Default);

	TAutoConsoleVariable<int32> CVarNVENCIntraRefreshCountFrames(
		TEXT("NVENC.IntraRefreshCountFrames"),
		0,
		TEXT("The total number of frames within the intra refresh period that should be used as 'intra refresh' frames. Smaller values make stream recovery quicker at the cost of more bandwidth usage. Default: 0."),
		ECVF_Default);

	TAutoConsoleVariable<bool> CVarNVENCKeyframeQPUseLastQP(
		TEXT("NVENC.KeyframeQPUseLastQP"),
		true,
		TEXT("If true QP of keyframes is no worse than the last frame transmitted (may cost latency), if false, it may be keyframe QP may be worse if network conditions require it (lower latency/worse quality). Default: true."),
		ECVF_Default);

	TAutoConsoleVariable<bool> CVarNVENCEnableStats(
		TEXT("NVENC.EnableStats"),
		false,
		TEXT("Whether to enable NVENC stats or not. Default: false."),
		ECVF_Default);

	TAutoConsoleVariable<int32> CVarNVENCKeyframeInterval(
		TEXT("NVENC.KeyframeInterval"),
		300,
		TEXT("Every N frames an IDR frame is sent. Default: 300. Note: A value <= 0 will disable sending of IDR frames on an interval."),
		ECVF_Default);

	template <typename T>
	void NVENCCommandLineParseValue(const TCHAR* Match, TAutoConsoleVariable<T>& CVar)
	{
		T Value;
		if (FParse::Value(FCommandLine::Get(), Match, Value))
		{
			CVar->Set(Value, ECVF_SetByCommandline);
		}
	};

	void NVENCCommandLineParseOption(const TCHAR* Match, TAutoConsoleVariable<bool>& CVar)
	{
		FString ValueMatch(Match);
		ValueMatch.Append(TEXT("="));
		FString Value;
		if (FParse::Value(FCommandLine::Get(), *ValueMatch, Value))
		{
			if (Value.Equals(FString(TEXT("true")), ESearchCase::IgnoreCase))
			{
				CVar->Set(true, ECVF_SetByCommandline);
			}
			else if (Value.Equals(FString(TEXT("false")), ESearchCase::IgnoreCase))
			{
				CVar->Set(false, ECVF_SetByCommandline);
			}
		}
		else if (FParse::Param(FCommandLine::Get(), Match))
		{
			CVar->Set(true, ECVF_SetByCommandline);
		}
	}

	void NVENCParseCommandLineFlags()
	{
		// CVar changes can only be triggered from the game thread

		AsyncTask(ENamedThreads::GameThread, []() {
			// Values
			NVENCCommandLineParseValue(TEXT("-NVENCIntraRefreshPeriodFrames="), CVarNVENCIntraRefreshPeriodFrames);
			NVENCCommandLineParseValue(TEXT("-NVENCIntraRefreshCountFrames="), CVarNVENCIntraRefreshCountFrames);
			NVENCCommandLineParseValue(TEXT("-NVENCKeyframeInterval="), CVarNVENCKeyframeInterval);

			// Options
			NVENCCommandLineParseOption(TEXT("-NVENCKeyFrameQPUseLastQP"), CVarNVENCKeyframeQPUseLastQP);
			NVENCCommandLineParseOption(TEXT("-NVENCEnableStats"), CVarNVENCEnableStats);

			// When NVENC stats CVar changes, change the output to screen flag.
			CVarNVENCEnableStats.AsVariable()->SetOnChangedCallback(FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* ChangedCVar) {
				FNVENCStats::Get().SetOutputToScreen(CVarNVENCEnableStats.GetValueOnAnyThread());
			}));

			if (CVarNVENCEnableStats.GetValueOnAnyThread())
			{
				FNVENCStats::Get().SetOutputToScreen(true);
			}
		});
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	static bool GetEncoderInfo(FNVENCCommon& NVENC, FVideoEncoderInfo& EncoderInfo);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	static int GetEncoderCapability(FNVENCCommon& NVENC, void* InEncoder, NV_ENC_CAPS InCapsToQuery);

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	bool FVideoEncoderNVENC_H264::GetIsAvailable(const FVideoEncoderInput& InVideoFrameFactory, FVideoEncoderInfo& OutEncoderInfo)
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		FNVENCCommon& NVENC = FNVENCCommon::Setup();
		bool bIsAvailable = NVENC.GetIsAvailable();
		if (bIsAvailable)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			OutEncoderInfo.CodecType = ECodecType::H264;
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
		
		return bIsAvailable;
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	void FVideoEncoderNVENC_H264::Register(FVideoEncoderFactory& InFactory)
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		FNVENCCommon& NVENC = FNVENCCommon::Setup();
		if (NVENC.GetIsAvailable() && IsRHIDeviceNVIDIA())
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			FVideoEncoderInfo EncoderInfo;
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			if (GetEncoderInfo(NVENC, EncoderInfo))
			{
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				InFactory.Register(EncoderInfo, []() { return TUniquePtr<FVideoEncoder>(new FVideoEncoderNVENC_H264()); });
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
			}
			
		}
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FVideoEncoderNVENC_H264::FVideoEncoderNVENC_H264()
		: NVENC(FNVENCCommon::Setup())
	{
		// Parse NVENC settings from command line (if any relevant ones are passed)
		NVENCParseCommandLineFlags();
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FVideoEncoderNVENC_H264::~FVideoEncoderNVENC_H264() { Shutdown(); }
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	bool FVideoEncoderNVENC_H264::Setup(TSharedRef<FVideoEncoderInput> InputFrameFactory, const FLayerConfig& InitConfig)
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		if (!NVENC.GetIsAvailable())
		{
			UE_LOGF(LogEncoderNVENC, Error, "NVENC not avaliable");
			return false;
		}

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FrameFormat = InputFrameFactory->GetFrameFormat();
		switch (FrameFormat)
		{
#if PLATFORM_WINDOWS
			case AVEncoder::EVideoFrameFormat::D3D11_R8G8B8A8_UNORM:
			case AVEncoder::EVideoFrameFormat::D3D12_R8G8B8A8_UNORM:
				EncoderDevice = InputFrameFactory->GetD3D11EncoderDevice();
				break;
#endif
			case AVEncoder::EVideoFrameFormat::CUDA_R8G8B8A8_UNORM:
				EncoderDevice = InputFrameFactory->GetCUDAEncoderContext();
				break;
			case AVEncoder::EVideoFrameFormat::Undefined:
			default:
				UE_LOGF(LogEncoderNVENC, Error, "Frame format %ls is not supported by NVENC_Encoder on this platform.", *ToString(FrameFormat));
				return false;
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		if (!EncoderDevice)
		{
			UE_LOGF(LogEncoderNVENC, Error, "NVENC needs encoder device.");
			return false;
		}

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FLayerConfig mutableConfig = InitConfig;
		if (mutableConfig.MaxFramerate == 0)
		{
			mutableConfig.MaxFramerate = 60;
		}

		return AddLayer(mutableConfig);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FVideoEncoder::FLayer* FVideoEncoderNVENC_H264::CreateLayer(uint32 layerIdx, FLayerConfig const& config)
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		auto const layer = new FNVENCLayer(layerIdx, config, *this);
		if (!layer->Setup())
		{
			delete layer;
			return nullptr;
		}
		return layer;
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	void FVideoEncoderNVENC_H264::DestroyLayer(FLayer* layer) { delete layer; }
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	void FVideoEncoderNVENC_H264::Encode(const TSharedPtr<FVideoEncoderInputFrame> InFrame, const FEncodeOptions& EncodeOptions)
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		for (FVideoEncoder::FLayer* Layer : Layers)
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		{
			FVideoEncoderNVENC_H264::FNVENCLayer* NVLayer = static_cast<FVideoEncoderNVENC_H264::FNVENCLayer*>(Layer);
			NVLayer->Encode(InFrame, EncodeOptions);
		}
	}

	void FVideoEncoderNVENC_H264::Flush()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		for (FVideoEncoder::FLayer* Layer : Layers)
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		{
			FVideoEncoderNVENC_H264::FNVENCLayer* NVLayer = static_cast<FVideoEncoderNVENC_H264::FNVENCLayer*>(Layer);
			NVLayer->Flush();
		}
	}

	void FVideoEncoderNVENC_H264::Shutdown()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		for (FVideoEncoder::FLayer* Layer : Layers)
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		{
			FVideoEncoderNVENC_H264::FNVENCLayer* NVLayer = static_cast<FVideoEncoderNVENC_H264::FNVENCLayer*>(Layer);
			NVLayer->Shutdown();
			DestroyLayer(NVLayer);
		}
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Layers.Reset();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	

	// --- FVideoEncoderNVENC_H264::FLayer ------------------------------------------------------------
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FVideoEncoderNVENC_H264::FNVENCLayer::FNVENCLayer(uint32 layerIdx, FLayerConfig const& config, FVideoEncoderNVENC_H264& encoder)
		: FLayer(config)
		, Encoder(encoder)
		, NVENC(FNVENCCommon::Setup())
		, CodecGUID(NV_ENC_CODEC_H264_GUID)
		, LayerIndex(layerIdx)
	{
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FVideoEncoderNVENC_H264::FNVENCLayer::~FNVENCLayer() {}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	bool FVideoEncoderNVENC_H264::FNVENCLayer::Setup()
	{
		if (CreateSession() && CreateInitialConfig())
		{
			// create encoder
			NVENCSTATUS Result = NVENC.nvEncInitializeEncoder(NVEncoder, &EncoderInitParams);
			if (Result != NV_ENC_SUCCESS)
			{
				UE_LOGF(LogEncoderNVENC, Error, "Unable to initialize NvEnc encoder (%ls).", *NVENC.GetErrorString(NVEncoder, Result));
				return false;
			}
			else
			{
				return true;
			}
		}

		return false;
	}

	bool FVideoEncoderNVENC_H264::FNVENCLayer::CreateSession()
	{
		if (!NVEncoder)
		{
			// create the encoder session
			NVENCStruct(NV_ENC_OPEN_ENCODE_SESSION_EX_PARAMS, OpenEncodeSessionExParams);
			OpenEncodeSessionExParams.apiVersion = NVENCAPI_VERSION;
			OpenEncodeSessionExParams.device = Encoder.EncoderDevice;

			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			switch (Encoder.FrameFormat)
			{
				case EVideoFrameFormat::D3D11_R8G8B8A8_UNORM:
				case EVideoFrameFormat::D3D12_R8G8B8A8_UNORM:
					OpenEncodeSessionExParams.deviceType = NV_ENC_DEVICE_TYPE_DIRECTX;
					break;
				case EVideoFrameFormat::CUDA_R8G8B8A8_UNORM:
					OpenEncodeSessionExParams.deviceType = NV_ENC_DEVICE_TYPE_CUDA;
					break;
				default:
					UE_LOGF(LogEncoderNVENC, Error, "FrameFormat %ls unavailable.", *ToString(Encoder.FrameFormat));
					return false;
			}
			PRAGMA_ENABLE_DEPRECATION_WARNINGS

			auto const result = NVENC.nvEncOpenEncodeSessionEx(&OpenEncodeSessionExParams, &NVEncoder);
			if (result != NV_ENC_SUCCESS)
			{
				UE_LOGF(LogEncoderNVENC, Error, "Unable to open NvEnc encoding session (%ls).", *NVENC.GetErrorString(NVEncoder, result));
				NVEncoder = nullptr;
				return false;
			}
		}

		return true;
	}

	bool FVideoEncoderNVENC_H264::FNVENCLayer::CreateInitialConfig()
	{
		// set the initialization parameters
		FMemory::Memzero(EncoderInitParams);

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		CurrentConfig.MaxFramerate = 60;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		EncoderInitParams.version = NV_ENC_INITIALIZE_PARAMS_VER;
		EncoderInitParams.encodeGUID = NV_ENC_CODEC_H264_GUID;
		EncoderInitParams.presetGUID = NV_ENC_PRESET_P4_GUID;
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		EncoderInitParams.frameRateNum = CurrentConfig.MaxFramerate;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		EncoderInitParams.frameRateDen = 1;
		EncoderInitParams.enablePTD = 1;
		EncoderInitParams.reportSliceOffsets = 0;
		EncoderInitParams.enableSubFrameWrite = 0;
		EncoderInitParams.maxEncodeWidth = 4096;
		EncoderInitParams.maxEncodeHeight = 4096;
		EncoderInitParams.tuningInfo = NV_ENC_TUNING_INFO_ULTRA_LOW_LATENCY;

		// load a preset configuration
		NVENCStruct(NV_ENC_PRESET_CONFIG, PresetConfig);
		PresetConfig.presetCfg.version = NV_ENC_CONFIG_VER;
		auto const result = NVENC.nvEncGetEncodePresetConfigEx(NVEncoder, EncoderInitParams.encodeGUID, EncoderInitParams.presetGUID, EncoderInitParams.tuningInfo, &PresetConfig);
		if (result != NV_ENC_SUCCESS)
		{
			UE_LOGF(LogEncoderNVENC, Error, "Unable to get NvEnc preset config (%ls).", *NVENC.GetErrorString(NVEncoder, result));
			return false;
		}

		// copy the preset config to our config
		FMemory::Memcpy(&EncoderConfig, &PresetConfig.presetCfg, sizeof(NV_ENC_CONFIG));
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		EncoderConfig.profileGUID = ConvertH264Profile(CurrentConfig.H264Profile);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		EncoderConfig.rcParams.version = NV_ENC_RC_PARAMS_VER;

		EncoderInitParams.encodeConfig = &EncoderConfig;

		////////////////////////////////
		// H.264 specific settings
		////////////////////////////////

		/*
		* Intra refresh - used to stabilise stream on the decoded side when frames are dropped/lost.
		*/
		int32 IntraRefreshPeriodFrames = CVarNVENCIntraRefreshPeriodFrames.GetValueOnAnyThread();
		int32 IntraRefreshCountFrames = CVarNVENCIntraRefreshCountFrames.GetValueOnAnyThread();
		bool bIntraRefreshSupported = GetEncoderCapability(NVENC, NVEncoder, NV_ENC_CAPS_SUPPORT_INTRA_REFRESH) > 0;
		bool bIntraRefreshEnabled = IntraRefreshPeriodFrames > 0;

		if (bIntraRefreshEnabled && bIntraRefreshSupported)
		{
			EncoderConfig.encodeCodecConfig.h264Config.enableIntraRefresh = 1;
			EncoderConfig.encodeCodecConfig.h264Config.intraRefreshPeriod = IntraRefreshPeriodFrames;
			EncoderConfig.encodeCodecConfig.h264Config.intraRefreshCnt = IntraRefreshCountFrames;

			UE_LOGF(LogEncoderNVENC, Log, "NVENC intra refresh enabled.");
			UE_LOGF(LogEncoderNVENC, Log, "NVENC intra refresh period set to = %d", IntraRefreshPeriodFrames);
			UE_LOGF(LogEncoderNVENC, Log, "NVENC intra refresh count = %d", IntraRefreshCountFrames);
		}
		else if (bIntraRefreshEnabled && !bIntraRefreshSupported)
		{
			UE_LOGF(LogEncoderNVENC, Error, "NVENC intra refresh capability is not supported on this device, cannot use this feature");
		}

		/*
		* IDR period - how often to send IDR (instantaneous decode refresh) frames, a.k.a keyframes. This can stabilise a stream that dropped/lost some frames (but at the cost of more bandwidth).
		*/
		int32 IdrPeriod = CVarNVENCKeyframeInterval.GetValueOnAnyThread();
		if (IdrPeriod > 0)
		{
			EncoderConfig.encodeCodecConfig.h264Config.idrPeriod = IdrPeriod;
		}

		/*
		* Repeat SPS/PPS - sends sequence and picture parameter info with every IDR frame - maximum stabilisation of the stream when IDR is sent.
		*/
		EncoderConfig.encodeCodecConfig.h264Config.repeatSPSPPS = 1;

		/*
		* Slice mode - set the slice mode to "entire frame as a single slice" because WebRTC implementation doesn't work well with slicing. The default slicing mode
		* produces (rarely, but especially under packet loss) grey full screen or just top half of it.
		*/
		EncoderConfig.encodeCodecConfig.h264Config.sliceMode = 0;
		EncoderConfig.encodeCodecConfig.h264Config.sliceModeData = 0;

		/*
		* These put extra meta data into the frames that allow Firefox to join mid stream.
		*/
		EncoderConfig.encodeCodecConfig.h264Config.outputFramePackingSEI = 1;
		EncoderConfig.encodeCodecConfig.h264Config.outputRecoveryPointSEI = 1;

		UpdateConfig();
		return true;
	}

	void FVideoEncoderNVENC_H264::FNVENCLayer::MaybeReconfigure()
	{
		uint64 PreReconfigureCycles = FPlatformTime::Cycles64();

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FScopeLock lock(&ConfigMutex);
		if (NeedsReconfigure)
		{
			UpdateConfig();

			if (EncoderInitParams.frameRateNum != CurrentConfig.MaxFramerate)
			{
				EncoderInitParams.frameRateNum = CurrentConfig.MaxFramerate;
			}

			if (CurrentConfig.Width != InputOutputBuffer->Width)
			{
				EncoderInitParams.encodeWidth = CurrentConfig.Width = InputOutputBuffer->Width;
			}
			if (CurrentConfig.Height != InputOutputBuffer->Height)
			{
				EncoderInitParams.encodeHeight = CurrentConfig.Height = InputOutputBuffer->Height;
			}

			NVENCStruct(NV_ENC_RECONFIGURE_PARAMS, ReconfigureParams);
			FMemory::Memcpy(&ReconfigureParams.reInitEncodeParams, &EncoderInitParams, sizeof(EncoderInitParams));

			auto const result = NVENC.nvEncReconfigureEncoder(NVEncoder, &ReconfigureParams);
			if (result != NV_ENC_SUCCESS)
				UE_LOGF(LogEncoderNVENC, Error, "Failed to update NVENC encoder configuration (%ls)", *NVENC.GetErrorString(NVEncoder, result));

			NeedsReconfigure = false;

			uint64 PostReconfigureCycles = FPlatformTime::Cycles64();

			if (CVarNVENCEnableStats.GetValueOnAnyThread())
			{
				// The time it took to do nvEncEncodePicture
				double ReconfigureDeltaMs = FPlatformTime::ToMilliseconds64(PostReconfigureCycles - PreReconfigureCycles);
				FNVENCStats::Get().SetReconfigureLatency(ReconfigureDeltaMs);
			}
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void FVideoEncoderNVENC_H264::FNVENCLayer::UpdateLastEncodedQP(uint32 InLastEncodedQP)
	{
		if (InLastEncodedQP == LastEncodedQP)
		{
			// QP is the same, do nothing.
			return;
		}

		LastEncodedQP = InLastEncodedQP;
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		NeedsReconfigure = true;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	

	void FVideoEncoderNVENC_H264::FNVENCLayer::UpdateConfig()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		EncoderInitParams.encodeWidth = EncoderInitParams.darWidth = CurrentConfig.Width;
		EncoderInitParams.encodeHeight = EncoderInitParams.darHeight = CurrentConfig.Height;

		uint32_t const MinQP = static_cast<uint32_t>(CurrentConfig.QPMin);
		uint32_t const MaxQP = static_cast<uint32_t>(CurrentConfig.QPMax);

		NV_ENC_RC_PARAMS& RateControlParams = EncoderInitParams.encodeConfig->rcParams;
		RateControlParams.rateControlMode = ConvertRateControlModeNVENC(CurrentConfig.RateControlMode);
		RateControlParams.averageBitRate = CurrentConfig.TargetBitrate > -1 ? CurrentConfig.TargetBitrate : DEFAULT_BITRATE;
		RateControlParams.maxBitRate = CurrentConfig.MaxBitrate > -1 ? CurrentConfig.MaxBitrate : DEFAULT_BITRATE; // Not used for CBR
		RateControlParams.multiPass = ConvertMultipassModeNVENC(CurrentConfig.MultipassMode);
		RateControlParams.minQP = { MinQP, MinQP, MinQP };
		RateControlParams.maxQP = { MaxQP, MaxQP, MaxQP };
		RateControlParams.enableMinQP = CurrentConfig.QPMin > -1;
		RateControlParams.enableMaxQP = CurrentConfig.QPMax > -1;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		// If we have QP ranges turned on use the last encoded QP to guide the max QP for an i-frame, so the i-frame doesn't look too blocky
		// Note: this does nothing if we have i-frames turned off.
		if (RateControlParams.enableMaxQP && LastEncodedQP > 0 && CVarNVENCKeyframeQPUseLastQP.GetValueOnAnyThread())
		{
			RateControlParams.maxQP.qpIntra = LastEncodedQP;
		}

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		EncoderInitParams.encodeConfig->profileGUID = ConvertH264Profile(CurrentConfig.H264Profile);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		
		NV_ENC_CONFIG_H264& H264Config = EncoderInitParams.encodeConfig->encodeCodecConfig.h264Config;
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		H264Config.enableFillerDataInsertion = CurrentConfig.FillData ? 1 : 0;

		if (CurrentConfig.RateControlMode == AVEncoder::FVideoEncoder::RateControlMode::CBR && CurrentConfig.FillData)
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		{
			// `outputPictureTimingSEI` is used in CBR mode to fill video frame with data to match the requested bitrate.
			H264Config.outputPictureTimingSEI = 1;
		}
	}

	void FVideoEncoderNVENC_H264::FNVENCLayer::EncodeBuffer(FInputOutput* Buffer)
	{
		checkf(Buffer, TEXT("Cannot encode null buffer."));
		if (Buffer == nullptr)
		{
			UE_LOGF(LogEncoderNVENC, Error, "Cannot proceed with encoding in NVENC - buffer was nullptr.");
			return;
		}

		if (Buffer && Buffer->PicParams.encodePicFlags == NV_ENC_PIC_FLAG_EOS)
		{
			NVENCSTATUS Result = NVENC.nvEncEncodePicture(NVEncoder, &(Buffer->PicParams));
			DestroyBuffer(Buffer);
			return;
		}

		uint64 StartQueueEncodeCycles = FPlatformTime::Cycles64();

		if (Buffer && !RegisterInputTexture(Buffer))
		{
			UE_LOGF(LogEncoderNVENC, Error, "Unable to register texture in NVENC");
			DestroyBuffer(Buffer);
			return;
		}

		Buffer->SubmitTimeCycles = StartQueueEncodeCycles;
		Buffer->PicParams.version = NV_ENC_PIC_PARAMS_VER;
		Buffer->PicParams.inputWidth = Buffer->Width;
		Buffer->PicParams.inputHeight = Buffer->Height;
		Buffer->PicParams.inputPitch = Buffer->Pitch ? Buffer->Pitch : Buffer->Width;
		Buffer->PicParams.inputBuffer = Buffer->MappedInput;
		Buffer->PicParams.bufferFmt = Buffer->BufferFormat;
		Buffer->PicParams.outputBitstream = Buffer->OutputBitstream;
		Buffer->PicParams.pictureStruct = NV_ENC_PIC_STRUCT_FRAME;

		MaybeReconfigure();

		// check if we have not reconfigued the buffer size
		if (Buffer->SourceFrame && (Buffer->Width != EncoderInitParams.encodeWidth || Buffer->Height != EncoderInitParams.encodeHeight))
		{
			DestroyBuffer(Buffer);
			return;
		}

		// Do synchronous encode
		uint64 PrenvEncEncodePictureCycles = FPlatformTime::Cycles64();
		NVENCSTATUS Result = NVENC.nvEncEncodePicture(NVEncoder, &(Buffer->PicParams));

		uint64 PostnvEncEncodePictureCycles = FPlatformTime::Cycles64();
		if (CVarNVENCEnableStats.GetValueOnAnyThread())
		{
			// The time it took to do nvEncEncodePicture
			double ProcessFramesFuncDeltaMs = FPlatformTime::ToMilliseconds64(PostnvEncEncodePictureCycles - PrenvEncEncodePictureCycles);
			FNVENCStats::Get().SetnvEncEncodePictureLatency(ProcessFramesFuncDeltaMs);
		}

		// We have an encoded bitstream - to use it we must bitstream lock, read, unlock (and then pass out the encoded output the whoever needs it)
		if (Result == NV_ENC_SUCCESS)
		{
			ProcessEncodedBuffer(Buffer);

			UnregisterInputTexture(Buffer);

			if (CVarNVENCEnableStats.GetValueOnAnyThread())
			{
				uint64 EndQueueEncodeCycles = FPlatformTime::Cycles64();
				double QueueEncodeMs = FPlatformTime::ToMilliseconds64(EndQueueEncodeCycles - StartQueueEncodeCycles);
				FNVENCStats::Get().SetQueueEncodeLatency(QueueEncodeMs);
			}

			return;
		}
		else
		{
			// Something went wrong
			UE_LOGF(LogEncoderNVENC, Error, "nvEncEncodePicture returned %ls", *NVENC.GetErrorString(NVEncoder, Result));

			if (Buffer)
			{
				DestroyBuffer(Buffer);
			}
		}
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	void FVideoEncoderNVENC_H264::FNVENCLayer::Encode(const TSharedPtr<FVideoEncoderInputFrame> InFrame, const FEncodeOptions& EncodeOptions)
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		// This encode is single threaded and blocking, if someone else is encoding right now, we wait.
		while (bIsProcessingFrame) 
		{
			// Encoder is already processing a frame, don't try and encode another
			return;
		}

		checkf(!bIsProcessingFrame, TEXT("NVENC encoder should only ever be called one thread at a time."));

		bIsProcessingFrame = true;

		FInputOutput* Buffer = GetOrCreateBuffer(StaticCastSharedPtr<FVideoEncoderInputFrameImpl>(InFrame));

		if (!Buffer)
		{
			UE_LOGF(LogEncoderNVENC, Error, "Cannot encode null buffer.");
			return;
		}

		// Clear pic params from last time this buffer was used
		Buffer->PicParams = {};
		Buffer->PicParams.encodePicFlags = 0;

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (EncodeOptions.bForceKeyFrame)
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		{
			LastKeyFrameTime = FDateTime::UtcNow();
			Buffer->PicParams.encodePicFlags |= NV_ENC_PIC_FLAG_FORCEIDR;
		}

		// Send SPS/PPS every frame (wasteful)
		//Buffer->PicParams.encodePicFlags |= NV_ENC_PIC_FLAG_OUTPUT_SPSPPS;

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Buffer->PicParams.frameIdx = InFrame->GetFrameID();
		Buffer->PicParams.inputTimeStamp = Buffer->TimeStamp = InFrame->GetTimestampUs();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		EncodeBuffer(Buffer);

		bIsProcessingFrame = false;
	}

	void FVideoEncoderNVENC_H264::FNVENCLayer::Flush()
	{
		DestroyBuffer(InputOutputBuffer);
	}

	void FVideoEncoderNVENC_H264::FNVENCLayer::Shutdown()
	{
		Flush();

		// Send an empty buffer through the encoder with the special "End of Stream (EOS)" flag set to shutdown properly.
		{
			FInputOutput* EmptyBuffer = CreateBuffer();

			if (!EmptyBuffer)
			{
				return;
			}

			EmptyBuffer->PicParams = {};
			EmptyBuffer->PicParams.version = NV_ENC_PIC_PARAMS_VER;
			EmptyBuffer->PicParams.encodePicFlags = NV_ENC_PIC_FLAG_EOS;

			EncodeBuffer(EmptyBuffer);
		}

		if (NVEncoder)
		{
			UE_LOGF(LogEncoderNVENC, Log, "Attempting to destroy NVENC encoder.");

			auto const result = NVENC.nvEncDestroyEncoder(NVEncoder);
			if (result != NV_ENC_SUCCESS)
			{
				UE_LOGF(LogEncoderNVENC, Error, "Failed to destroy NVENC encoder (%ls).", *NVENC.GetErrorString(NVEncoder, result));
			}
			else
			{
				UE_LOGF(LogEncoderNVENC, Log, "Successfully destroyed NVENC encoder.");
			}
			NVEncoder = nullptr;
		}
	}

	void FVideoEncoderNVENC_H264::FNVENCLayer::ProcessEncodedBuffer(FInputOutput* Buffer)
	{
		uint64 StartProcessFramesFuncCycles;
		uint64 FinishedEncodingCycles;

		StartProcessFramesFuncCycles = FPlatformTime::Cycles64();

		// lock output buffers for CPU access
		if (LockOutputBuffer(Buffer))
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			if (Encoder.OnEncodedPacket)
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			{
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				FCodecPacket Packet = FCodecPacket::Create(static_cast<const uint8*>(Buffer->BitstreamData), Buffer->BitstreamDataSize);
				PRAGMA_ENABLE_DEPRECATION_WARNINGS

				if (Buffer->PictureType & NV_ENC_PIC_TYPE_IDR)
				{
					UE_LOGF(LogEncoderNVENC, Verbose, "Generated IDR Frame");
					PRAGMA_DISABLE_DEPRECATION_WARNINGS
					Packet.IsKeyFrame = true;
					PRAGMA_ENABLE_DEPRECATION_WARNINGS
				}
				
				else
				{
					// If it is not a keyframe store the QP.
					UpdateLastEncodedQP(Buffer->FrameAvgQP);
				}

				FinishedEncodingCycles = FPlatformTime::Cycles64();
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				Packet.VideoQP = Buffer->FrameAvgQP;
				Packet.Timings.StartTs = FTimespan::FromMilliseconds(FPlatformTime::ToMilliseconds64(Buffer->SubmitTimeCycles));
				Packet.Timings.FinishTs = FTimespan::FromMilliseconds(FPlatformTime::ToMilliseconds64(FinishedEncodingCycles));
				Packet.Framerate = EncoderInitParams.frameRateNum;
				PRAGMA_ENABLE_DEPRECATION_WARNINGS

				// Stats
				if (CVarNVENCEnableStats.GetValueOnAnyThread())
				{
					// The time it took to do just the encode part
					double ProcessFramesFuncDeltaMs = FPlatformTime::ToMilliseconds64(FinishedEncodingCycles - StartProcessFramesFuncCycles);
					FNVENCStats::Get().SetProcessFramesFuncLatency(ProcessFramesFuncDeltaMs);

					// Total time the encoder took to process a frame from submit through to encode
					PRAGMA_DISABLE_DEPRECATION_WARNINGS
					double TotalEncoderLatencyMs = Packet.Timings.FinishTs.GetTotalMilliseconds() - Packet.Timings.StartTs.GetTotalMilliseconds();
					PRAGMA_ENABLE_DEPRECATION_WARNINGS
					FNVENCStats::Get().SetTotalEncoderLatency(TotalEncoderLatencyMs);
				}

				// Send frame to OnEncodedPacket listener, such as WebRTC.
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				if (Encoder.OnEncodedPacket)
				{
					Encoder.OnEncodedPacket(LayerIndex, Buffer->SourceFrame, Packet);
				}
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
			}

			if (!UnlockOutputBuffer(Buffer))
			{
				UE_LOGF(LogEncoderNVENC, Error, "Failed to unlock output buffer.");
				DestroyBuffer(Buffer);
			}
		}
		else
		{
			UE_LOGF(LogEncoderNVENC, Error, "Failed to lock output buffer.");
			DestroyBuffer(Buffer);
		}
	}

	int FVideoEncoderNVENC_H264::FNVENCLayer::GetCapability(NV_ENC_CAPS CapsToQuery) const
	{
		int CapsValue = 0;
		NVENCStruct(NV_ENC_CAPS_PARAM, CapsParam);
		CapsParam.capsToQuery = CapsToQuery;
		auto const result = NVENC.nvEncGetEncodeCaps(NVEncoder, CodecGUID, &CapsParam, &CapsValue);
		if (result != NV_ENC_SUCCESS)
		{
			UE_LOGF(LogEncoderNVENC, Warning, "Failed to query for NVENC capability %d (%ls).", CapsToQuery, *NVENC.GetErrorString(NVEncoder, result));
			return 0;
		}
		return CapsValue;
	}

	FVideoEncoderNVENC_H264::FNVENCLayer::FInputOutput* FVideoEncoderNVENC_H264::FNVENCLayer::GetOrCreateBuffer(const TSharedPtr<FVideoEncoderInputFrameImpl> InFrame)
	{

		if (InputOutputBuffer == nullptr)
		{
			InputOutputBuffer = CreateBuffer();
		}
		else
		{
			// Check for buffer and Frame texture resolution mismatch
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			if (InFrame->GetWidth() != InputOutputBuffer->Width || InFrame->GetHeight() != InputOutputBuffer->Height)
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			{
				DestroyBuffer(InputOutputBuffer);
				InputOutputBuffer = CreateBuffer();
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				NeedsReconfigure = true;
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
			}
			
		}

		InputOutputBuffer->SourceFrame = InFrame;
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		InputOutputBuffer->Width = InFrame->GetWidth();
		InputOutputBuffer->Height = InFrame->GetHeight();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		return InputOutputBuffer;
	}

	FVideoEncoderNVENC_H264::FNVENCLayer::FInputOutput* FVideoEncoderNVENC_H264::FNVENCLayer::CreateBuffer()
	{
		FInputOutput* Buffer = new FInputOutput();

		// output bit stream buffer
		NVENCStruct(NV_ENC_CREATE_BITSTREAM_BUFFER, CreateParam);
		{
			auto const result = NVENC.nvEncCreateBitstreamBuffer(NVEncoder, &CreateParam);
			if (result != NV_ENC_SUCCESS)
			{
				UE_LOGF(LogEncoderNVENC, Error, "Failed to create NVENC output buffer (%ls).", *NVENC.GetErrorString(NVEncoder, result));
				DestroyBuffer(Buffer);
				return nullptr;
			}
		}

		Buffer->OutputBitstream = CreateParam.bitstreamBuffer;

		return Buffer;
	}

	void FVideoEncoderNVENC_H264::FNVENCLayer::DestroyBuffer(FInputOutput* InBuffer)
	{
		checkf(InBuffer, TEXT("Cannot destroy buffer in NVENC - buffer was nullptr."));
		if (!InBuffer)
		{
			return;
		}

		// Release the AVEncoder frame
		if (InBuffer->SourceFrame)
		{
			InBuffer->SourceFrame->Release();
		}

		// Unregister input texture - if any
		UnregisterInputTexture(InBuffer);

		// Destroy output buffer - if any
		UnlockOutputBuffer(InBuffer);
		if (InBuffer->OutputBitstream)
		{
			auto const result = NVENC.nvEncDestroyBitstreamBuffer(NVEncoder, InBuffer->OutputBitstream);
			if (result != NV_ENC_SUCCESS)
			{
				UE_LOGF(LogEncoderNVENC, Warning, "Failed to destroy NVENC output buffer (%ls).", *NVENC.GetErrorString(NVEncoder, result));
			}
			InBuffer->OutputBitstream = nullptr;
		}

		// Clear source texture
		InBuffer->InputTexture = nullptr;

		delete InBuffer;
		InBuffer = nullptr;	
	}

	void FVideoEncoderNVENC_H264::FNVENCLayer::CreateResourceDIRECTX(FInputOutput* InBuffer, NV_ENC_REGISTER_RESOURCE& RegisterParam, FIntPoint TextureSize)
	{
		checkf(InBuffer, TEXT("Cannot create DirectX resource in NVENC - buffer was nullptr."));
		if (!InBuffer)
		{
			return;
		}

#if PLATFORM_WINDOWS
		D3D11_TEXTURE2D_DESC Desc;
		static_cast<ID3D11Texture2D*>(InBuffer->InputTexture)->GetDesc(&Desc);

		switch (Desc.Format)
		{
			case DXGI_FORMAT_NV12:
				InBuffer->BufferFormat = NV_ENC_BUFFER_FORMAT_NV12;
				break;
			case DXGI_FORMAT_R8G8B8A8_UNORM:
				InBuffer->BufferFormat = NV_ENC_BUFFER_FORMAT_ABGR;
				break;
			case DXGI_FORMAT_B8G8R8A8_UNORM:
				InBuffer->BufferFormat = NV_ENC_BUFFER_FORMAT_ARGB;
				break;
			default:
				UE_LOGF(LogEncoderNVENC, Error, "Invalid input texture format for NVENC (%d)", Desc.Format);
				return;
		}

		InBuffer->Width = TextureSize.X;
		InBuffer->Height = TextureSize.Y;
		InBuffer->Pitch = 0;

		RegisterParam.resourceType = NV_ENC_INPUT_RESOURCE_TYPE_DIRECTX;
		RegisterParam.width = Desc.Width;
		RegisterParam.height = Desc.Height;
		RegisterParam.pitch = InBuffer->Pitch;
		RegisterParam.bufferFormat = InBuffer->BufferFormat;
		RegisterParam.bufferUsage = NV_ENC_INPUT_IMAGE;
#endif
	}

	void FVideoEncoderNVENC_H264::FNVENCLayer::CreateResourceCUDAARRAY(FInputOutput* InBuffer, NV_ENC_REGISTER_RESOURCE& RegisterParam, FIntPoint TextureSize)
	{
		checkf(InBuffer, TEXT("Cannot create CUDA array in NVENC - buffer was nullptr."));
		if (!InBuffer)
		{
			return;
		}

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		switch (InBuffer->SourceFrame->GetCUDA().UnderlyingRHI)
		{
			case FVideoEncoderInputFrame::EUnderlyingRHI::Vulkan:
				InBuffer->Pitch = TextureSize.X * GPixelFormats[EPixelFormat::PF_A8R8G8B8].BlockBytes;
				break;
			case FVideoEncoderInputFrame::EUnderlyingRHI::D3D11:
			case FVideoEncoderInputFrame::EUnderlyingRHI::D3D12:
				InBuffer->Pitch = 0;
				break;
			default:
				break;
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		InBuffer->Width = TextureSize.X;
		InBuffer->Height = TextureSize.Y;
		InBuffer->BufferFormat = NV_ENC_BUFFER_FORMAT_ARGB;

		RegisterParam.resourceType = NV_ENC_INPUT_RESOURCE_TYPE_CUDAARRAY;
		RegisterParam.width = InBuffer->Width;
		RegisterParam.height = InBuffer->Height;
		RegisterParam.pitch = InBuffer->Pitch;
		RegisterParam.bufferFormat = InBuffer->BufferFormat;
		RegisterParam.bufferUsage = NV_ENC_INPUT_IMAGE;
	}

	bool FVideoEncoderNVENC_H264::FNVENCLayer::RegisterInputTexture(FInputOutput* InBuffer)
	{
		checkf(InBuffer, TEXT("Cannot register texture in NVENC - buffer was nullptr."));
		if (!InBuffer)
		{
			return false;
		}

		if (!InBuffer->InputTexture && InBuffer->PicParams.encodePicFlags != NV_ENC_PIC_FLAG_EOS)
		{
			checkf(InBuffer->SourceFrame, TEXT("Cannot register texture if source frame is null."));

			void* TextureToCompress = nullptr;

			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			switch (InBuffer->SourceFrame->GetFormat())
			{
#if PLATFORM_WINDOWS
				case EVideoFrameFormat::D3D11_R8G8B8A8_UNORM:
					TextureToCompress = InBuffer->SourceFrame->GetD3D11().EncoderTexture;
					break;
				case EVideoFrameFormat::D3D12_R8G8B8A8_UNORM:
					UE_LOGF(LogEncoderNVENC, Error, "NVENC got passed a DX12 texture which it does not support, make sure it has been shared with a DX11 context and pass that texture instead.");
					return false; // NVENC cant encode a D3D12 texture directly this should be converted to D3D11
#endif
				case EVideoFrameFormat::CUDA_R8G8B8A8_UNORM:
					TextureToCompress = InBuffer->SourceFrame->GetCUDA().EncoderTexture;
					break;
				case AVEncoder::EVideoFrameFormat::Undefined:
				default:
					break;
			}
			PRAGMA_ENABLE_DEPRECATION_WARNINGS

			if (!TextureToCompress)
			{
				UE_LOGF(LogEncoderNVENC, Fatal, "Got passed a null texture to encode.");
				return false;
			}

			InBuffer->InputTexture = TextureToCompress;
			NVENCStruct(NV_ENC_REGISTER_RESOURCE, RegisterParam);

			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			FIntPoint TextureSize(InBuffer->SourceFrame->GetWidth(), InBuffer->SourceFrame->GetHeight());

			switch (InBuffer->SourceFrame->GetFormat())
			{
#if PLATFORM_WINDOWS
				case EVideoFrameFormat::D3D11_R8G8B8A8_UNORM:
				case EVideoFrameFormat::D3D12_R8G8B8A8_UNORM:
					CreateResourceDIRECTX(InBuffer, RegisterParam, TextureSize);
					break;
#endif
				case EVideoFrameFormat::CUDA_R8G8B8A8_UNORM:
					CreateResourceCUDAARRAY(InBuffer, RegisterParam, TextureSize);
					break;
				case AVEncoder::EVideoFrameFormat::Undefined:
				default:
					break;
			}
			PRAGMA_ENABLE_DEPRECATION_WARNINGS

			RegisterParam.resourceToRegister = InBuffer->InputTexture;

			NVENCSTATUS Result = NVENC.nvEncRegisterResource(NVEncoder, &RegisterParam);
			if (Result != NV_ENC_SUCCESS)
			{
				UE_LOGF(LogEncoderNVENC, Error, "Failed to register input texture with NVENC (%ls)", *NVENC.GetErrorString(NVEncoder, Result));
				return false;
			}
			InBuffer->RegisteredInput = RegisterParam.registeredResource;

			if (!MapInputTexture(InBuffer))
			{
				UE_LOGF(LogEncoderNVENC, Error, "Could not map input texture, cannot proceed with encoding this buffer.");
				return false;
			}
		}

		return true;
	}

	bool FVideoEncoderNVENC_H264::FNVENCLayer::UnregisterInputTexture(FInputOutput* InBuffer)
	{
		checkf(InBuffer, TEXT("Cannot unregister texture in NVENC - buffer was nullptr."));
		if (!InBuffer)
		{
			return false;
		}

		if (!UnmapInputTexture(InBuffer))
		{
			UE_LOGF(LogEncoderNVENC, Error, "Failed to unmap input texture.");
		}

		if (InBuffer->RegisteredInput)
		{
			NVENCSTATUS Result = NVENC.nvEncUnregisterResource(NVEncoder, InBuffer->RegisteredInput);
			if (Result != NV_ENC_SUCCESS)
			{
				UE_LOGF(LogEncoderNVENC, Error, "Failed to unregister input texture with NVENC (%ls)", *NVENC.GetErrorString(NVEncoder, Result));
				InBuffer->InputTexture = nullptr;
				InBuffer->RegisteredInput = nullptr;
				return false;
			}
			InBuffer->InputTexture = nullptr;
			InBuffer->RegisteredInput = nullptr;
		}
		return true;
	}

	bool FVideoEncoderNVENC_H264::FNVENCLayer::MapInputTexture(FInputOutput* InBuffer)
	{
		checkf(InBuffer, TEXT("Cannot map texture in NVENC - buffer was nullptr."));
		if (!InBuffer)
		{
			return false;
		}

		if (!InBuffer->MappedInput)
		{
			NVENCStruct(NV_ENC_MAP_INPUT_RESOURCE, MapInputResource);
			MapInputResource.registeredResource = InBuffer->RegisteredInput;
			NVENCSTATUS Result = NVENC.nvEncMapInputResource(NVEncoder, &MapInputResource);
			if (Result != NV_ENC_SUCCESS)
			{
				UE_LOGF(LogEncoderNVENC, Error, "Failed to map input texture buffer (%ls)", *NVENC.GetErrorString(NVEncoder, Result));
				return false;
			}
			InBuffer->MappedInput = MapInputResource.mappedResource;
			check(InBuffer->BufferFormat == MapInputResource.mappedBufferFmt);
		}
		return true;
	}

	bool FVideoEncoderNVENC_H264::FNVENCLayer::UnmapInputTexture(FInputOutput* InBuffer)
	{
		checkf(InBuffer, TEXT("Cannot unmap texture in NVENC - buffer was nullptr."));
		if (!InBuffer)
		{
			return false;
		}

		if (InBuffer->MappedInput)
		{
			NVENCSTATUS Result = NVENC.nvEncUnmapInputResource(NVEncoder, InBuffer->MappedInput);
			if (Result != NV_ENC_SUCCESS)
			{
				UE_LOGF(LogEncoderNVENC, Error, "Failed to unmap input texture buffer (%ls)", *NVENC.GetErrorString(NVEncoder, Result));
				InBuffer->MappedInput = nullptr;
				return false;
			}
			InBuffer->MappedInput = nullptr;
		}
		return true;
	}

	bool FVideoEncoderNVENC_H264::FNVENCLayer::LockOutputBuffer(FInputOutput* InBuffer)
	{
		checkf(InBuffer, TEXT("Cannot lock output buffer in NVENC - buffer was nullptr."));
		if (!InBuffer)
		{
			return false;
		}

		if (!InBuffer->BitstreamData)
		{
			uint64 PreLockCycles = FPlatformTime::Cycles64();

			// lock output buffers for CPU access
			NVENCStruct(NV_ENC_LOCK_BITSTREAM, LockBitstreamParam);
			LockBitstreamParam.outputBitstream = InBuffer->OutputBitstream;
			NVENCSTATUS Result = NVENC.nvEncLockBitstream(NVEncoder, &LockBitstreamParam);

			if (CVarNVENCEnableStats.GetValueOnAnyThread())
			{
				uint64 PostLockCycles = FPlatformTime::Cycles64();
				double LockLatencyMs = FPlatformTime::ToMilliseconds64(PostLockCycles - PreLockCycles);
				FNVENCStats::Get().SetLockOutputBufferLatency(LockLatencyMs);
			}

			if (Result != NV_ENC_SUCCESS)
			{
				UE_LOGF(LogEncoderNVENC, Error, "Failed to lock output bitstream for NVENC encoder (%ls).", *NVENC.GetErrorString(NVEncoder, Result));
				return false;
			}
			else
			{
				InBuffer->BitstreamData = LockBitstreamParam.bitstreamBufferPtr;
				InBuffer->BitstreamDataSize = LockBitstreamParam.bitstreamSizeInBytes;
				InBuffer->PictureType = LockBitstreamParam.pictureType;
				InBuffer->FrameAvgQP = LockBitstreamParam.frameAvgQP;
				InBuffer->TimeStamp = LockBitstreamParam.outputTimeStamp;
			}
		}
		return true;
	}

	bool FVideoEncoderNVENC_H264::FNVENCLayer::UnlockOutputBuffer(FInputOutput* InBuffer)
	{
		checkf(InBuffer, TEXT("Cannot unregister texture NVENC - buffer was nullptr."));
		if (!InBuffer)
		{
			return false;
		}

		if (InBuffer->BitstreamData)
		{
			uint64 PreUnlockCycles = FPlatformTime::Cycles64();

			NVENCSTATUS Result = NVENC.nvEncUnlockBitstream(NVEncoder, InBuffer->OutputBitstream);

			if (CVarNVENCEnableStats.GetValueOnAnyThread())
			{
				uint64 PostUnlockCycles = FPlatformTime::Cycles64();
				double UnlockLatencyMs = FPlatformTime::ToMilliseconds64(PostUnlockCycles - PreUnlockCycles);
				FNVENCStats::Get().SetUnlockOutputBufferLatency(UnlockLatencyMs);
			}

			if (Result != NV_ENC_SUCCESS)
			{
				UE_LOGF(LogEncoderNVENC, Error, "Failed to unlock output bitstream for NVENC encoder (%ls).", *NVENC.GetErrorString(NVEncoder, Result));
				return false;
			}
			else
			{
				InBuffer->BitstreamData = nullptr;
				InBuffer->BitstreamDataSize = 0;
			}
		}
		return true;
	}

#if PLATFORM_WINDOWS
	static bool CreateEncoderDevice(TRefCountPtr<ID3D11Device>& OutEncoderDevice, TRefCountPtr<ID3D11DeviceContext>& OutEncoderDeviceContext)
	{
		// need a d3d11 context to be able to set up encoder
		TRefCountPtr<IDXGIFactory1> DXGIFactory1;
		TRefCountPtr<IDXGIAdapter> Adapter;

		HRESULT Result = CreateDXGIFactory1(__uuidof(IDXGIFactory1), (void**)DXGIFactory1.GetInitReference());
		if (Result != S_OK)
		{
			UE_LOGF(LogEncoderNVENC, Error, "Failed to create DX factory for NVENC.");
			return false;
		}

		for (int GpuIndex = 0; GpuIndex < MAX_GPU_INDEXES; GpuIndex++)
		{
			if ((Result = DXGIFactory1->EnumAdapters(GpuIndex, Adapter.GetInitReference())) != S_OK)
			{
				UE_LOGF(LogEncoderNVENC, Error, "Failed to enum GPU #%d for NVENC.", GpuIndex);
				return false;
			}

			DXGI_ADAPTER_DESC AdapterDesc;
			Adapter->GetDesc(&AdapterDesc);
			if (AdapterDesc.VendorId != 0x10DE) // NVIDIA
			{
				continue;
			}

			if ((Result = D3D11CreateDevice(Adapter, D3D_DRIVER_TYPE_UNKNOWN, NULL, 0, NULL, 0, D3D11_SDK_VERSION, OutEncoderDevice.GetInitReference(), NULL,
					 OutEncoderDeviceContext.GetInitReference()))
				!= S_OK)
			{
				UE_LOGF(LogEncoderNVENC, Error, "Failed to create D3D11 device for NVENC.");
			}
			else
			{
				UE_LOGF(LogEncoderNVENC, Log, "Created D3D11 device for NVENC on '%ls'.", AdapterDesc.Description);
				return true;
			}
		}

		UE_LOGF(LogEncoderNVENC, Error, "No compatible devices found for NVENC.");
		return false;
	}
#endif

#if PLATFORM_WINDOWS
	static void* CreateEncoderSession(FNVENCCommon& NVENC, TRefCountPtr<ID3D11Device> InD3D11Device)
	{
		void* EncoderSession = nullptr;
		// create the encoder session
		NVENCStruct(NV_ENC_OPEN_ENCODE_SESSION_EX_PARAMS, OpenEncodeSessionExParams);
		OpenEncodeSessionExParams.device = InD3D11Device;
		OpenEncodeSessionExParams.deviceType = NV_ENC_DEVICE_TYPE_DIRECTX; // Currently only DX11 is supported
		OpenEncodeSessionExParams.apiVersion = NVENCAPI_VERSION;

		NVENCSTATUS NvResult = NVENC.nvEncOpenEncodeSessionEx(&OpenEncodeSessionExParams, &EncoderSession);
		// UE_LOGF(LogEncoderNVENC, Error, "NVENC.nvEncOpenEncodeSessionEx(&OpenEncodeSessionExParams, &EncoderSession); -> %d", NvResult);
		if (NvResult != NV_ENC_SUCCESS)
		{
			UE_LOGF(LogEncoderNVENC, Error, "Unable to open NvEnc encoding session (status: %d).", NvResult);
			EncoderSession = nullptr;
		}
		return EncoderSession;
	}
#endif // PLATFORM_WINDOWS

	static void* CreateEncoderSession(FNVENCCommon& NVENC, CUcontext CudaContext)
	{
		void* EncoderSession = nullptr;
		// create the encoder session
		NVENCStruct(NV_ENC_OPEN_ENCODE_SESSION_EX_PARAMS, OpenEncodeSessionExParams);
		OpenEncodeSessionExParams.device = CudaContext;
		OpenEncodeSessionExParams.deviceType = NV_ENC_DEVICE_TYPE_CUDA; // We use cuda to pass vulkan device memory to nvenc
		OpenEncodeSessionExParams.apiVersion = NVENCAPI_VERSION;

		NVENCSTATUS NvResult = NVENC.nvEncOpenEncodeSessionEx(&OpenEncodeSessionExParams, &EncoderSession);
		//	UE_LOGF(LogEncoderNVENC, Error, "NVENC.nvEncOpenEncodeSessionEx(&OpenEncodeSessionExParams, &EncoderSession); -> %d", NvResult);
		if (NvResult != NV_ENC_SUCCESS)
		{
			UE_LOGF(LogEncoderNVENC, Error, "Unable to open NvEnc encoding session (status: %d).", NvResult);
			EncoderSession = nullptr;
		}
		return EncoderSession;
	}

	static int GetEncoderCapability(FNVENCCommon& NVENC, void* InEncoder, NV_ENC_CAPS InCapsToQuery)
	{
		int CapsValue = 0;
		NVENCStruct(NV_ENC_CAPS_PARAM, CapsParam);
		CapsParam.capsToQuery = InCapsToQuery;
		NVENCSTATUS Result = NVENC.nvEncGetEncodeCaps(InEncoder, NV_ENC_CODEC_H264_GUID, &CapsParam, &CapsValue);

		if (Result != NV_ENC_SUCCESS)
		{
			UE_LOGF(LogEncoderNVENC, Warning, "Failed to query for NVENC capability %d (error %d).", InCapsToQuery, Result);
			return 0;
		}
		return CapsValue;
	}

	static bool GetEncoderSupportedProfiles(FNVENCCommon& NVENC, void* InEncoder, uint32& OutSupportedProfiles)
	{
		const uint32 MaxProfileGUIDs = 32;
		GUID ProfileGUIDs[MaxProfileGUIDs];
		uint32 NumProfileGUIDs = 0;

		OutSupportedProfiles = 0;
		NVENCSTATUS Result = NVENC.nvEncGetEncodeProfileGUIDs(InEncoder, NV_ENC_CODEC_H264_GUID, ProfileGUIDs, MaxProfileGUIDs, &NumProfileGUIDs);

		if (Result != NV_ENC_SUCCESS)
		{
			UE_LOGF(LogEncoderNVENC, Error, "Unable to query profiles supported by NvEnc (error: %d).", Result);
			return false;
		}
		for (uint32 Index = 0; Index < NumProfileGUIDs; ++Index)
		{
			if (memcmp(&NV_ENC_H264_PROFILE_BASELINE_GUID, &ProfileGUIDs[Index], sizeof(GUID)) == 0)
			{
				OutSupportedProfiles |= H264Profile_Baseline;
				if (GetEncoderCapability(NVENC, InEncoder, NV_ENC_CAPS_SUPPORT_CONSTRAINED_ENCODING))
				{
					OutSupportedProfiles |= H264Profile_ConstrainedBaseline;
				}
			}
			else if (memcmp(&NV_ENC_H264_PROFILE_MAIN_GUID, &ProfileGUIDs[Index], sizeof(GUID)) == 0)
			{
				OutSupportedProfiles |= H264Profile_Main;
			}
			else if (memcmp(&NV_ENC_H264_PROFILE_HIGH_GUID, &ProfileGUIDs[Index], sizeof(GUID)) == 0)
			{
				OutSupportedProfiles |= H264Profile_High;
			}
			else if (memcmp(&NV_ENC_H264_PROFILE_CONSTRAINED_HIGH_GUID, &ProfileGUIDs[Index], sizeof(GUID)) == 0)
			{
				OutSupportedProfiles |= H264Profile_ConstrainedHigh;
			}
		}
		return OutSupportedProfiles != 0;
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	static bool GetEncoderSupportedInputFormats(FNVENCCommon& NVENC, void* InEncoder, TArray<EVideoFrameFormat>& OutSupportedInputFormats)
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		const uint32_t MaxInputFmtCount = 32;
		uint32_t InputFmtCount = 0;
		NV_ENC_BUFFER_FORMAT InputFormats[MaxInputFmtCount];
		NVENCSTATUS Result = NVENC.nvEncGetInputFormats(InEncoder, NV_ENC_CODEC_H264_GUID, InputFormats, MaxInputFmtCount, &InputFmtCount);
		if (Result != NV_ENC_SUCCESS)
		{
			UE_LOGF(LogEncoderNVENC, Error, "Unable to query input formats supported by NvEnc (error: %d).", Result);
			return false;
		}

		for (uint32_t Index = 0; Index < InputFmtCount; ++Index)
		{
			switch (InputFormats[Index])
			{
				case NV_ENC_BUFFER_FORMAT_IYUV:
					break;
				case NV_ENC_BUFFER_FORMAT_NV12:
					break;
				case NV_ENC_BUFFER_FORMAT_ARGB:
					break;
				case NV_ENC_BUFFER_FORMAT_ABGR:
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
#if PLATFORM_WINDOWS
					OutSupportedInputFormats.Push(EVideoFrameFormat::D3D11_R8G8B8A8_UNORM);
					OutSupportedInputFormats.Push(EVideoFrameFormat::D3D12_R8G8B8A8_UNORM);
#endif
					OutSupportedInputFormats.Push(EVideoFrameFormat::CUDA_R8G8B8A8_UNORM);
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
					break;
			}
		}
		return true;
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	static bool GetEncoderInfo(FNVENCCommon& NVENC, FVideoEncoderInfo& EncoderInfo)
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		bool bSuccess = true;

		// create a temporary encoder session
		void* EncoderSession = nullptr;

#if PLATFORM_WINDOWS
		// if we are under windows we can create a temporary dx11 device to get back infomation about the encoder
		TRefCountPtr<ID3D11Device> EncoderDevice;
		TRefCountPtr<ID3D11DeviceContext> EncoderDeviceContext;

		if (!CreateEncoderDevice(EncoderDevice, EncoderDeviceContext))
		{
			bSuccess = false;
		}

		if ((EncoderSession = CreateEncoderSession(NVENC, EncoderDevice)) == nullptr)
		{
			bSuccess = false;
		}
#endif
		// if we dont already have an encoder session try with CUDA if its avaliable
		if (!EncoderSession && FModuleManager::GetModulePtr<FCUDAModule>("CUDA"))
		{
			FCUDAModule& CUDAModule = FModuleManager::GetModuleChecked<FCUDAModule>("CUDA");
			if (CUDAModule.IsAvailable())
			{
				EncoderSession = CreateEncoderSession(NVENC, CUDAModule.GetCudaContext());
				bSuccess = EncoderSession != nullptr;
			}
			else
			{
				bSuccess = false;
			}
		}

		// if we dont have a session by now opt out. this will cause NVENC to not register
		if (!EncoderSession || !bSuccess)
		{
			return false;
		}

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		EncoderInfo.CodecType = ECodecType::H264;
		EncoderInfo.MaxWidth = GetEncoderCapability(NVENC, EncoderSession, NV_ENC_CAPS_WIDTH_MAX);
		EncoderInfo.MaxHeight = GetEncoderCapability(NVENC, EncoderSession, NV_ENC_CAPS_HEIGHT_MAX);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		int LevelMax = GetEncoderCapability(NVENC, EncoderSession, NV_ENC_CAPS_LEVEL_MAX);
		int LevelMin = GetEncoderCapability(NVENC, EncoderSession, NV_ENC_CAPS_LEVEL_MIN);
		if (LevelMin > 0 && LevelMax > 0 && LevelMax >= LevelMin)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			EncoderInfo.H264.MinLevel = (LevelMin > 9) ? LevelMin : 9;
			EncoderInfo.H264.MaxLevel = (LevelMax < 9) ? 9 : (LevelMax > NV_ENC_LEVEL_H264_52) ? NV_ENC_LEVEL_H264_52
																							   : LevelMax;
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
		else
		{
			UE_LOGF(LogEncoderNVENC, Error, "Failed to query min/max h264 level supported by NvEnc (reported min/max=%d/%d).", LevelMin, LevelMax);
			bSuccess = false;
		}

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (!GetEncoderSupportedProfiles(NVENC, EncoderSession, EncoderInfo.H264.SupportedProfiles) || !GetEncoderSupportedInputFormats(NVENC, EncoderSession, EncoderInfo.SupportedInputFormats))
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		{
			bSuccess = false;
		}

		// destroy encoder session
		if (EncoderSession)
		{
			NVENC.nvEncDestroyEncoder(EncoderSession);
		}

		return bSuccess;
	}

} /* namespace AVEncoder */

#undef MIN_UPDATE_FRAMERATE_SECS
