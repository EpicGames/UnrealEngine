// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/ThreadSafeBool.h"
#include "Misc/DateTime.h"
#include "VideoCommon.h"
#include "VideoEncoder.h"
#include <nvEncodeAPI.h>

namespace AVEncoder { class FNVENCCommon; }
namespace AVEncoder { class FVideoEncoderFactory; }
namespace AVEncoder { class FVideoEncoderInputFrameImpl; }

namespace AVEncoder
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	class FVideoEncoderNVENC_H264 : public FVideoEncoder
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{

	public:
		virtual ~FVideoEncoderNVENC_H264() override; 

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		bool Setup(TSharedRef<FVideoEncoderInput> InputFrameFactory, const FLayerConfig& InitConfig) override;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		void Shutdown() override;

		// Query whether or not encoder is supported and available
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		static bool GetIsAvailable(const FVideoEncoderInput& InFrameFactory, FVideoEncoderInfo& OutEncoderInfo);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		// Register encoder with video encoder factory
		static void Register(FVideoEncoderFactory& InFactory);

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		void Encode(const TSharedPtr<FVideoEncoderInputFrame> InFrame, const FEncodeOptions& EncodeOptions) override;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		void Flush();

	protected:
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FLayer* CreateLayer(uint32 InLayerIndex, const FLayerConfig& InLayerConfig) override;
		void DestroyLayer(FLayer* layer) override;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

	private:
		FVideoEncoderNVENC_H264();

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		class FNVENCLayer : public FLayer
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		{

		public:
			struct FInputOutput
			{
				TSharedPtr<AVEncoder::FVideoEncoderInputFrameImpl> SourceFrame = nullptr;
				void* InputTexture = nullptr;
				uint32 Width = 0;
				uint32 Height = 0;
				uint32 Pitch = 0;
				NV_ENC_BUFFER_FORMAT BufferFormat = NV_ENC_BUFFER_FORMAT_UNDEFINED;
				NV_ENC_REGISTERED_PTR RegisteredInput = nullptr;
				NV_ENC_INPUT_PTR MappedInput = nullptr;
				NV_ENC_PIC_PARAMS PicParams = {};
				NV_ENC_OUTPUT_PTR OutputBitstream = nullptr;
				const void* BitstreamData = nullptr;
				uint32 BitstreamDataSize = 0;
				NV_ENC_PIC_TYPE PictureType = NV_ENC_PIC_TYPE_UNKNOWN;
				uint32 FrameAvgQP = 0;
				uint64 TimeStamp;
				uint64 SubmitTimeCycles;
			};

			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			FNVENCLayer(uint32 layerIdx, FLayerConfig const& config, FVideoEncoderNVENC_H264& encoder);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			~FNVENCLayer();
			bool Setup();
			bool CreateSession();
			bool CreateInitialConfig();
			int GetCapability(NV_ENC_CAPS CapsToQuery) const;
			FString GetError(NVENCSTATUS ForStatus) const;
			void MaybeReconfigure();
			void UpdateConfig();
			void UpdateLastEncodedQP(uint32 InLastEncodedQP);
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			void Encode(const TSharedPtr<FVideoEncoderInputFrame> InFrame, const FEncodeOptions& EncodeOptions);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			void EncodeBuffer(FInputOutput* Buffer);
			void ProcessEncodedBuffer(FInputOutput* Buffer);
			void Flush();
			void Shutdown();
			void UpdateBitrate(uint32 InMaxBitRate, uint32 InTargetBitRate);
			void UpdateResolution(uint32 InMaxBitRate, uint32 InTargetBitRate);
			FInputOutput* GetOrCreateBuffer(const TSharedPtr<FVideoEncoderInputFrameImpl> InFrame);
			FInputOutput* CreateBuffer();
			void DestroyBuffer(FInputOutput* InBuffer);
			bool RegisterInputTexture(FInputOutput* InBuffer);
			bool UnregisterInputTexture(FInputOutput* InBuffer);
			bool MapInputTexture(FInputOutput* InBuffer);
			bool UnmapInputTexture(FInputOutput* InBuffer);
			bool LockOutputBuffer(FInputOutput* InBuffer);
			bool UnlockOutputBuffer(FInputOutput* InBuffer);
			void CreateResourceDIRECTX(FInputOutput* InBuffer, NV_ENC_REGISTER_RESOURCE& RegisterParam, FIntPoint TextureSize);
			void CreateResourceCUDAARRAY(FInputOutput* InBuffer, NV_ENC_REGISTER_RESOURCE& RegisterParam, FIntPoint TextureSize);

		public:
			FVideoEncoderNVENC_H264& Encoder;
			FNVENCCommon& NVENC;
			GUID CodecGUID;
			uint32 LayerIndex;
			void* NVEncoder = nullptr;
			NV_ENC_INITIALIZE_PARAMS EncoderInitParams;
			NV_ENC_CONFIG EncoderConfig;
			FDateTime LastKeyFrameTime = 0;
			bool bForceNextKeyframe = false;
			uint32 LastEncodedQP = 0;

		private:
			FThreadSafeBool bIsProcessingFrame = false;
			FInputOutput* InputOutputBuffer = nullptr;
		};

		FNVENCCommon& NVENC;
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		EVideoFrameFormat FrameFormat = EVideoFrameFormat::Undefined;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		// this could be a TRefCountPtr<ID3D11Device>, CUcontext or void*
		void* EncoderDevice;
	};

} // namespace AVEncoder
