// Copyright Epic Games, Inc. All Rights Reserved.


#include "CudaModule.h"
#include "IVulkanDynamicRHI.h"

#include "Misc/App.h"
#include "NVENC_Common.h"
#include "NVENC_EncoderH264.h"
#include "VideoEncoderFactory.h"

#include <vulkan_core.h>

class FNVENCEncoderModule : public IModuleInterface
{
public:
	void StartupModule()
	{
		using namespace AVEncoder;
		if (FApp::CanEverRender())
		{
			FNVENCCommon& NVENC = FNVENCCommon::Setup();

			if (NVENC.GetIsAvailable())
			{
				FModuleManager::LoadModuleChecked<FCUDAModule>("CUDA").OnPostCUDAInit.AddLambda([]()
				{
					if (IsRHIDeviceNVIDIA())
					{
						PRAGMA_DISABLE_DEPRECATION_WARNINGS
						FVideoEncoderNVENC_H264::Register(FVideoEncoderFactory::Get());
						PRAGMA_ENABLE_DEPRECATION_WARNINGS
					}
				});
			}
		}
	}
};

IMPLEMENT_MODULE(FNVENCEncoderModule, EncoderNVENC);
